// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/ToolMenusTestUtilities.h"

#include "Misc/UObjectTestUtils.h"

namespace UE::ToolMenus
{
	FName FMenuEntry::TypeName = "MenuEntry";
	FName FMenuWildcardEntry::TypeName = "WildcardMenuEntry";
	FName FMenu::TypeName = "Menu";

	bool FToolMenuAutomationTestAdapter::Matches(const FMenuEntryBase* InExpectedEntry, const FToolMenuEntry& InActualEntry)
	{
		// Empty name matches anything
		if (InExpectedEntry->Name.IsNone())
		{
			return true;
		}

		if (!TestInstance.TestEqual("Entry Name", InActualEntry.Name.ToString(), InExpectedEntry->Name.ToString()))
		{
			return false;
		}

		return true;
	}

	bool FToolMenuAutomationTestAdapter::Matches(const FMenuEntry& InExpectedEntry, const FToolMenuEntry& InActualEntry)
	{
		if (!Matches(static_cast<const FMenuEntryBase*>(&InExpectedEntry), InActualEntry))
		{
			return false;
		}

		if (InExpectedEntry.Type.IsSet() && !UE::CoreUObject::TestEqual(FString::Printf(TEXT("Entry (\"%s\") MultiBlockType"), *InActualEntry.Name.ToString()), InActualEntry.Type, InExpectedEntry.Type.GetValue(), TestInstance))
		{
			return false;
		}

		return true;
	}

	bool FToolMenuAutomationTestAdapter::Matches(const FMenuSection& InExpectedSection, const FToolMenuSection& InActualSection, const bool bInTestChildren)
	{
		bool bAllSucceeded = true;

		if (!InExpectedSection.Name.IsNone()
			&& !TestInstance.TestEqual("Section Name", InActualSection.Name, InExpectedSection.Name))
		{
			bAllSucceeded = false;
		}

		if (!bInTestChildren)
		{
			return true;
		}

		// If we don't require all entries, we default to failure, and only succeed if we find all entries in the (expected) section
		if (!Parameters.EntryMatchParameters.bActualHasAllExpectedItems)
		{
			bAllSucceeded = false;
		}

		// If we require an exact match, we require an exact count
		const bool bRequireMatchingEntryCount = Parameters.EntryMatchParameters.bActualHasAllExpectedItems && Parameters.EntryMatchParameters.bExpectedHasAllActualItems;
		if (bRequireMatchingEntryCount
			&& !TestInstance.TestEqual(
				FString::Printf(TEXT("Entry Count (for section \"%s\")"), *InActualSection.Name.ToString()),
				InActualSection.Blocks.Num(),
				InExpectedSection.Entries.Num()))
		{
			// ...we default to failure but continue checking for matches before returning
			bAllSucceeded = false;
		}

		// Actual should have at least the expected item count
		if (Parameters.EntryMatchParameters.bActualHasAllExpectedItems
			&& !TestInstance.TestGreaterEqual(
				FString::Printf(TEXT("Entry Count (for section \"%s\")"), *InActualSection.Name.ToString()),
				InActualSection.Blocks.Num(),
				InExpectedSection.Entries.Num()))
		{
			// ...we default to failure but continue checking for matches before returning
			bAllSucceeded = false;
		}

		// Expected should have at least the actual item count
		if (Parameters.EntryMatchParameters.bExpectedHasAllActualItems
			&& !TestInstance.TestGreaterEqual(
				FString::Printf(TEXT("Entry Count (for section \"%s\")"), *InActualSection.Name.ToString()),
				InExpectedSection.Entries.Num(),
				InActualSection.Blocks.Num()))
		{
			// ...we default to failure but continue checking for matches before returning
			bAllSucceeded = false;
		}

		// If we don't require all entries to be specified in the expected section, keep track of the amount we expect to find and only succeed if we find them all
		const int32 ExpectedEntryCount = InExpectedSection.Entries.Num();
		int32 FoundEntryCount = 0;

		// Track the last found index, to validate order if required
		int32 LastFoundEntryIdx = INDEX_NONE;
		bool bFoundEntriesInOrder = true;

		for (int32 ExpectedEntryIdx = 0; ExpectedEntryIdx < InExpectedSection.Entries.Num(); ++ExpectedEntryIdx)
		{
			const TSharedRef<FMenuEntryBase>& ExpectedEntry = InExpectedSection.Entries[ExpectedEntryIdx];

			for (int32 ActualEntryIdx = 0; ActualEntryIdx < InActualSection.Blocks.Num(); ++ActualEntryIdx)
			{
				if (!TestValidIndex(FString::Printf(TEXT("Entry (for section \"%s\")"), *InActualSection.Name.ToString()), ActualEntryIdx, InActualSection.Blocks))
				{
					bAllSucceeded = false;
					continue;
				}

				const FToolMenuEntry& ActualEntry = InActualSection.Blocks[ActualEntryIdx];

				// "None" matches anything, so we're a bit looser about matching requirements
				const bool bMatchAny = ExpectedEntry->Name.IsNone();

				if (bMatchAny || ExpectedEntry->Name == ActualEntry.Name)
				{
					if (!bMatchAny && ExpectedEntry->GetTypeName() == FMenuEntry::TypeName)
					{
						if (!Matches(StaticCastSharedRef<FMenuEntry>(ExpectedEntry).Get(), ActualEntry))
						{
							bAllSucceeded = false;
						}
					}

					if (!bMatchAny && !Matches(&ExpectedEntry.Get(), ActualEntry))
					{
						bAllSucceeded = false;
					}
					else
					{
						// We matched, increment found
						++FoundEntryCount;

						if (Parameters.EntryMatchParameters.bActualHasExpectedOrder)
						{
							if (ActualEntryIdx < LastFoundEntryIdx)
							{
								bFoundEntriesInOrder = false;
							}

							LastFoundEntryIdx = ActualEntryIdx;
						}

						break;
					}
				}
			}
		}

		if (!bAllSucceeded)
		{
			const bool bFoundExpectedSections = (!Parameters.EntryMatchParameters.bActualHasAllExpectedItems && FoundEntryCount > 0)
											|| FoundEntryCount >= ExpectedEntryCount;

			bAllSucceeded = bFoundExpectedSections;
		}
		
		// Finally, check if we found all entries in order if required
		bAllSucceeded = bAllSucceeded && (!Parameters.EntryMatchParameters.bActualHasExpectedOrder || bFoundEntriesInOrder);

 		return bAllSucceeded;
	}

	bool FToolMenuAutomationTestAdapter::Matches(const FMenu& InExpectedMenu, const UToolMenu& InActualMenu)
	{
		bool bAllSucceeded = true;

		if (!InExpectedMenu.Name.IsNone()
			&& !TestInstance.TestEqual("ToolMenu Name", InActualMenu.MenuName, InExpectedMenu.Name))
		{
			bAllSucceeded = false;
		}

		// If we don't require all sections, we default to failure, and only succeed if we find all sections in the (expected) menu
		if (!Parameters.SectionMatchParameters.bActualHasAllExpectedItems)
		{
			bAllSucceeded = false;
		}

		// If we require an exact match, we require an exact count
		const bool bRequireMatchingSectionCount = Parameters.SectionMatchParameters.bActualHasAllExpectedItems && Parameters.SectionMatchParameters.bExpectedHasAllActualItems;
		if (bRequireMatchingSectionCount
			&& !TestInstance.TestEqual(
				FString::Printf(TEXT("Section Count (for menu \"%s\")"), *InActualMenu.MenuName.ToString()),
				InActualMenu.Sections.Num(),
				InExpectedMenu.Sections.Num()))
		{
			// ...we default to failure but continue checking for matches before returning
			bAllSucceeded = false;
		}

		// Actual should have at least the expected item count
		if (Parameters.SectionMatchParameters.bActualHasAllExpectedItems
			&& !TestInstance.TestGreaterEqual(
				FString::Printf(TEXT("Section Count (for menu \"%s\")"), *InActualMenu.MenuName.ToString()),
				InActualMenu.Sections.Num(),
				InExpectedMenu.Sections.Num()))
		{
			// ...we default to failure but continue checking for matches before returning
			bAllSucceeded = false;
		}

		// Expected should have at least the actual item count
		if (Parameters.SectionMatchParameters.bExpectedHasAllActualItems
			&& !TestInstance.TestGreaterEqual(
				FString::Printf(TEXT("Section Count (for menu \"%s\")"), *InExpectedMenu.Name.ToString()),
				InExpectedMenu.Sections.Num(),
				InActualMenu.Sections.Num()))
		{
			// ...we default to failure but continue checking for matches before returning
			bAllSucceeded = false;
		}

		// If we don't require all sections to be specified in the expected menu, keep track of the amount we expect to find and only succeed if we find them all
		const int32 ExpectedSectionCount = InExpectedMenu.Sections.Num();
		int32 FoundSectionCount = 0;

		// Track the last found index, to validate order if required
		int32 LastFoundSectionIdx = INDEX_NONE;
		bool bFoundEntriesInOrder = true;

		for (int32 ExpectedSectionIdx = 0; ExpectedSectionIdx < InExpectedMenu.Sections.Num(); ++ExpectedSectionIdx)
		{
			const TSharedRef<FMenuSection>& ExpectedSection = InExpectedMenu.Sections[ExpectedSectionIdx];

			for (int32 ActualSectionIdx = 0; ActualSectionIdx < InActualMenu.Sections.Num(); ++ActualSectionIdx)
			{
				if (!TestValidIndex(FString::Printf(TEXT("Section (for menu \"%s\")"), *InActualMenu.MenuName.ToString()), ActualSectionIdx, InActualMenu.Sections))
				{
					bAllSucceeded = false;
					continue;
				}

				const FToolMenuSection& ActualSection = InActualMenu.Sections[ActualSectionIdx];

				// "None" matches anything, so we're a bit looser about matching requirements
				const bool bMatchAny = ExpectedSection->Name.IsNone();

				// "None" matches anything
				if (bMatchAny || ExpectedSection->Name == ActualSection.Name)
				{
					if (!bMatchAny && !Matches(*ExpectedSection, ActualSection, Parameters.SectionMatchParameters.bMatchChildrenIfAnyExpected && ExpectedSection->Entries.Num() > 0))
					{
						bAllSucceeded = false;
					}
					else
					{
						// We matched, increment found
						++FoundSectionCount;

						if (Parameters.SectionMatchParameters.bActualHasExpectedOrder)
						{
							if (ActualSectionIdx < LastFoundSectionIdx)
							{
								bFoundEntriesInOrder = false;
							}

							LastFoundSectionIdx = ActualSectionIdx;
						}

						break;
					}
				}
			}
		}

		if (!bAllSucceeded)
		{
			const bool bFoundExpectedSections = (!Parameters.SectionMatchParameters.bActualHasAllExpectedItems && FoundSectionCount > 0)
											|| FoundSectionCount >= ExpectedSectionCount;

			bAllSucceeded = bFoundExpectedSections;
		}

		// Finally, check if we found all entries in order if required
		bAllSucceeded = bAllSucceeded && (!Parameters.SectionMatchParameters.bActualHasExpectedOrder || bFoundEntriesInOrder);

		return bAllSucceeded;
	}
}
