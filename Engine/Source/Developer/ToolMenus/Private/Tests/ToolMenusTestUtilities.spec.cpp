// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Misc/AutomationTest.h"
#include "Tests/ToolMenusTestUtilities.h"

namespace UE::ToolMenus::Private
{
	BEGIN_DEFINE_SPEC(FToolMenusTestUtiltiesSpec, "System.ToolMenus.TestUtilities", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	static const FName TestMenuName;
	void RegisterTestMenu();
	const UToolMenu* GenerateTestMenu() const;
	bool TestMatch(const FString& InWhat, const FMenu& InExpectedMenu, const UToolMenu& InActualMenu, const FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters& InParameters = {});
	bool TestMismatch(const FString& InWhat, const FMenu& InExpectedMenu, const UToolMenu& InActualMenu, const FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters& InParameters = {});
	END_DEFINE_SPEC(FToolMenusTestUtiltiesSpec)

	const FName FToolMenusTestUtiltiesSpec::TestMenuName = "TestMenu";

	void FToolMenusTestUtiltiesSpec::Define()
	{
		RegisterTestMenu();

		Describe("Exact Match", [this]()
		{
			It("Should match the exact sections", [this]()
			{
				const FMenu ExpectedMenuStructure(
					"TestMenu",
					{
						Section("FirstSection"),
						Section("MiddleSection"),
						Section("LastSection")
					});

				const UToolMenu* ActualMenu = GenerateTestMenu();

				FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters TestAdapterParameters;
				TestAdapterParameters.SectionMatchParameters.bActualHasAllExpectedItems = true;
				TestAdapterParameters.EntryMatchParameters.bActualHasAllExpectedItems = true;

				TestMatch(
					"Actual menu sections to (exactly) match the expected sections",
					ExpectedMenuStructure,
					*ActualMenu,
					TestAdapterParameters);
			});

			It("Should match the exact entries", [this]()
			{
				const FMenu ExpectedMenuStructure(
					"TestMenu",
					{
						Section("FirstSection", {
							Entry("FirstEntry"),
							Entry("MiddleEntry"),
							Separator(),
							Entry("LastEntry")
						}),
						Section("MiddleSection"),
						Section("LastSection")
					});

				const UToolMenu* ActualMenu = GenerateTestMenu();

				FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters TestAdapterParameters;
				TestAdapterParameters.SectionMatchParameters.bActualHasAllExpectedItems = false;
				TestAdapterParameters.EntryMatchParameters.bActualHasAllExpectedItems = true;

				TestMatch(
					"Actual section entries to (exactly) match the expected section entries",
					ExpectedMenuStructure,
					*ActualMenu,
					TestAdapterParameters);
			});
		});

		Describe("Partial Match", [this]()
		{
			It("Should match any existing section", [this]()
			{
				const FMenu ExpectedPartialMenuStructure(
					"TestMenu",
					{
						Section("MiddleSection"),
						Section("SectionThatDoesntExist")
					});

				const UToolMenu* ActualMenu = GenerateTestMenu();

				FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters TestAdapterParameters;
				TestAdapterParameters.SectionMatchParameters.bActualHasAllExpectedItems = false;
				TestAdapterParameters.EntryMatchParameters.bActualHasAllExpectedItems = false;

				TestMatch(
					"Actual menu sections to (partially) match the expected section(s)",
					ExpectedPartialMenuStructure,
					*ActualMenu,
					TestAdapterParameters);
			});

			It("Should allow extra sections", [this]()
			{
				const FMenu ExpectedPartialMenuStructure(
					"TestMenu",
					{
						Section("MiddleSection")
					});

				const UToolMenu* ActualMenu = GenerateTestMenu();

				FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters TestAdapterParameters;
				TestAdapterParameters.SectionMatchParameters.bActualHasAllExpectedItems = true;
				TestAdapterParameters.EntryMatchParameters.bActualHasAllExpectedItems = false;

				TestMatch(
					"Actual menu sections to (partially) match the expected section(s)",
					ExpectedPartialMenuStructure,
					*ActualMenu,
					TestAdapterParameters);
			});

			It("Shouldn't match out-of-order sections", [this]()
			{
				const FMenu WrongOrderMenuStructure(
					"TestMenu",
					{
						Section("FirstSection"),
						Section("LastSection"),
						Section("MiddleSection")
					});

				const UToolMenu* ActualMenu = GenerateTestMenu();
				FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters TestAdapterParameters;
				TestAdapterParameters.SectionMatchParameters.bActualHasAllExpectedItems = true; // otherwise the out-of-order section would be considered extra, and valid
				TestAdapterParameters.SectionMatchParameters.bActualHasExpectedOrder = true;
				TestAdapterParameters.EntryMatchParameters.bActualHasAllExpectedItems = false;

				TestMismatch(
					"Menu sections shouldn't match wrongly ordered sections",
					WrongOrderMenuStructure,
					*ActualMenu,
					TestAdapterParameters);
			});

			It("Should match any existing entry", [this]()
			{
				const FMenu ExpectedPartialMenuStructure(
					"TestMenu",
					{
						Section("FirstSection", {
							Entry("MiddleEntry"),
							Entry("LastEntry")
						})
					});

				const UToolMenu* ActualMenu = GenerateTestMenu();

				FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters TestAdapterParameters;
				TestAdapterParameters.SectionMatchParameters.bActualHasAllExpectedItems = false;
				TestAdapterParameters.EntryMatchParameters.bActualHasAllExpectedItems = false;

				TestMatch(
					"Actual menu entries to (partially) match the expected entries(s)",
					ExpectedPartialMenuStructure,
					*ActualMenu,
					TestAdapterParameters);
			});

			It("Shouldn't match out-of-order entries", [this]()
			{
				const FMenu ExpectedPartialMenuStructure(
					"TestMenu",
					{
						Section("FirstSection", {
							Entry("LastEntry"),
							Entry("MiddleEntry")
						})
					});

				const UToolMenu* ActualMenu = GenerateTestMenu();

				FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters TestAdapterParameters;
				TestAdapterParameters.SectionMatchParameters.bActualHasAllExpectedItems = true; // otherwise the out-of-order entry would be considered extra, and valid
				TestAdapterParameters.EntryMatchParameters.bActualHasAllExpectedItems = false;
				TestAdapterParameters.EntryMatchParameters.bActualHasExpectedOrder = true;

				TestMismatch(
					"Menu entries shouldn't match wrongly ordered entries",
					ExpectedPartialMenuStructure,
					*ActualMenu,
					TestAdapterParameters);
			});
		});
	}

	// Registers the actual menu to test against
	void FToolMenusTestUtiltiesSpec::RegisterTestMenu()
	{
		if (!UToolMenus::Get()->IsMenuRegistered(TestMenuName))
		{
			UToolMenu* TestMenu = UToolMenus::Get()->RegisterMenu(TestMenuName);

			auto MakeEntry = [](const FName Name)
			{
				return FToolMenuEntry::InitMenuEntry(Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FUIAction(), EUserInterfaceActionType::Button);
			};

			FToolMenuSection& FirstSection = TestMenu->AddSection("FirstSection");
			{
				FirstSection.AddEntry(MakeEntry("FirstEntry"));
				FirstSection.AddEntry(MakeEntry("MiddleEntry"));
				FirstSection.AddSeparator("FirstSeparator");
				FirstSection.AddEntry(MakeEntry("LastEntry"));
			}

			FToolMenuSection& MiddleSection = TestMenu->AddSection("MiddleSection");

			FToolMenuSection& LastSection = TestMenu->AddSection("LastSection");
		}
	}

	const UToolMenu* FToolMenusTestUtiltiesSpec::GenerateTestMenu() const
	{
		FToolMenuContext ToolMenuContext;
		return UToolMenus::Get()->GenerateMenu(TestMenuName, ToolMenuContext);
	}

	bool FToolMenusTestUtiltiesSpec::TestMatch(
		const FString& InWhat,
		const FMenu& InExpectedMenu,
		const UToolMenu& InActualMenu,
		const FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters& InParameters)
	{
		return TestTrue(
			InWhat,
			FToolMenuAutomationTestAdapter(*this, InParameters)
			.Matches(InExpectedMenu, InActualMenu));
	}

	bool FToolMenusTestUtiltiesSpec::TestMismatch(
		const FString& InWhat,
		const FMenu& InExpectedMenu,
		const UToolMenu& InActualMenu,
		const FToolMenuAutomationTestAdapter::FToolMenuAutomationTestAdapterParameters& InParameters)
	{
		return TestFalse(
			InWhat,
			FToolMenuAutomationTestAdapter(*this, InParameters)
			.Matches(InExpectedMenu, InActualMenu));
	}
}

#endif
