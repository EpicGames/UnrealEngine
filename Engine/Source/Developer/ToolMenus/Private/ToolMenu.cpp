// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenu.h"
#include "ToolMenus.h"
#include "IToolMenusModule.h"

#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Styling/SlateStyleRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolMenu)

UToolMenu::UToolMenu() :
	MenuType(EMultiBoxType::Menu)
	, bShouldCleanupContextOnDestroy(true)
	, bShouldCloseWindowAfterMenuSelection(true)
	, bCloseSelfOnly(false)
	, bSearchable(true)
	, bToolBarIsFocusable(false)
	, bToolBarForceSmallIcons(false)
	, bRegistered(false)
	, bIsRegistering(false)
	, bExtendersEnabled(true)
	, MaxHeight(1000)
{
}

void UToolMenu::InitMenu(const FToolMenuOwner InOwner, FName InName, FName InParent, EMultiBoxType InType)
{
	MenuOwner = InOwner;
	MenuName = InName;
	MenuParent = InParent;
	MenuType = InType;
}

FReply UToolMenu::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	for (int32 i = 0; i < Sections.Num(); ++i)
	{
		for (FToolMenuEntry& Entry : Sections[i].Blocks)
		{
			if (Entry.Type == EMultiBlockType::ToolBarButton
				&& Entry.IsCommandKeybindOnly())
			{
				if (Entry.CommandAcceptsInput(InKeyEvent))
				{
					if (Entry.TryExecuteToolUIAction(Context))
					{
						return FReply::Handled();
					}
				}
			}
		}
	}
	return FReply::Unhandled();
}

const ISlateStyle* UToolMenu::GetStyleSet() const
{
	if (!StyleSetName.IsNone())
	{
		const ISlateStyle* FoundSlateStyleSet = FSlateStyleRegistry::FindSlateStyle(StyleSetName);
		ensureMsgf(FoundSlateStyleSet, TEXT("Slate Style Set not found: '%s' for menu: '%s'."), *StyleSetName.ToString(), *GetMenuName().ToString());
		return FoundSlateStyleSet;
	}

	return &FCoreStyle::Get();
}

void UToolMenu::SetStyleSet(const ISlateStyle* InStyleSet)
{
	const FName NewStyleSetName = InStyleSet ? InStyleSet->GetStyleSetName() : NAME_None;
	if (NewStyleSetName != StyleName)
	{
		StyleSetName = NewStyleSetName;
	}
}

void UToolMenu::InitGeneratedCopy(const UToolMenu* Source, const FName InMenuName, const FToolMenuContext* InContext)
{
	// Skip sections

	MenuName = InMenuName;
	MenuParent = Source->MenuParent;
	StyleName = Source->StyleName;
	TutorialHighlightName = Source->TutorialHighlightName;
	MenuType = Source->MenuType;
	StyleSetName = Source->StyleSetName;
	bShouldCloseWindowAfterMenuSelection = Source->bShouldCloseWindowAfterMenuSelection;
	bCloseSelfOnly = Source->bCloseSelfOnly;
	bSearchable = Source->bSearchable;
	bSeparateSections = Source->bSeparateSections;
	bToolBarIsFocusable = Source->bToolBarIsFocusable;
	bToolBarForceSmallIcons = Source->bToolBarForceSmallIcons;
	MenuOwner = Source->MenuOwner;

	SubMenuParent = Source->SubMenuParent;
	SubMenuSourceEntryName = Source->SubMenuSourceEntryName;
	MaxHeight = Source->MaxHeight;
	bExtendersEnabled = Source->bExtendersEnabled;

	MaxHeight = Source->MaxHeight;
	if (InContext)
	{
		Context = *InContext;
	}
}

int32 UToolMenu::IndexOfSection(const FName InSectionName) const
{
	for (int32 i=0; i < Sections.Num(); ++i)
	{
		if (Sections[i].Name == InSectionName)
		{
			return i;
		}
	}

	return INDEX_NONE;
}

// Note: This function is very similar to FToolMenuSection::FindBlockInsertIndex.
int32 UToolMenu::FindInsertIndex(const FToolMenuSection& InSection) const
{
	const FToolMenuInsert InInsertPosition = InSection.InsertPosition;

	// Insert a Default-positioned section after all First and Default-positioned sections but before any
	// Last-positioned sections.
	if (InInsertPosition.IsDefault())
	{
		for (int32 i = 0; i < Sections.Num(); ++i)
		{
			if (Sections[i].InsertPosition.Position == EToolMenuInsertType::Last)
			{
				return i;
			}
		}

		return Sections.Num();
	}

	// Insert a First-positioned section after any other First-positioned sections but before all Default and
	// Last-positioned sections.
	if (InInsertPosition.Position == EToolMenuInsertType::First)
	{
		for (int32 i = 0; i < Sections.Num(); ++i)
		{
			if (Sections[i].InsertPosition.Position != InInsertPosition.Position)
			{
				return i;
			}
		}

		return Sections.Num();
	}

	// Insert a Last-positioned section after all other sections, include other Last-positioned sections.
	if (InInsertPosition.Position == EToolMenuInsertType::Last)
	{
		for (int32 i = Sections.Num() - 1; i >= 0; --i)
		{
			if (Sections[i].InsertPosition.Position == InInsertPosition.Position)
			{
				return i + 1;
			}
		}

		return Sections.Num();
	}

	int32 DestIndex = IndexOfSection(InInsertPosition.Name);
	if (DestIndex == INDEX_NONE)
	{
		return DestIndex;
	}

	if (InInsertPosition.Position == EToolMenuInsertType::After)
	{
		++DestIndex;

		// Insert after the final entry that has the exact same InsertPosition
		for (int32 i = DestIndex; i < Sections.Num(); ++i)
		{
			if (Sections[i].InsertPosition == InInsertPosition)
			{
				DestIndex = i + 1;
				// Do not break because EToolMenuInsertType::Before may have been used
			}
		}
	}

	for (int32 i = DestIndex; i < Sections.Num(); ++i)
	{
		if (Sections[i].InsertPosition != InInsertPosition)
		{
			return i;
		}
	}

	return Sections.Num();
}

FToolMenuSection& UToolMenu::AddDynamicSection(const FName SectionName, const FNewSectionConstructChoice& InConstruct, const FToolMenuInsert InPosition)
{
	FToolMenuSection& Section = AddSection(SectionName, TAttribute< FText >(), InPosition);
	Section.Construct = InConstruct;
	return Section;
}

bool UToolMenu::IsRegistering() const
{
	return bIsRegistering;
}

FToolMenuSection& UToolMenu::AddSection(const FName SectionName, const TAttribute< FText >& InLabel, const FToolMenuInsert InPosition)
{
	int32 InsertIndex = (SectionName != NAME_None) ? IndexOfSection(SectionName) : INDEX_NONE;
	if (InsertIndex != INDEX_NONE)
	{
		if (InLabel.IsSet())
		{
			Sections[InsertIndex].Label = InLabel;
		}

		if (InPosition.Name != NAME_None || InPosition.Position != EToolMenuInsertType::Default)
		{
			Sections[InsertIndex].InsertPosition = InPosition;
		}

		// Sort registered sections to appear before unregistered
		if (IsRegistering() && !Sections[InsertIndex].bAddedDuringRegister)
		{
			Sections[InsertIndex].bAddedDuringRegister = true;

			for (int32 i = 0; i < InsertIndex; ++i)
			{
				if (!Sections[i].bAddedDuringRegister)
				{
					FToolMenuSection RemovedSection;
					Swap(Sections[InsertIndex], RemovedSection);
					Sections.Insert(MoveTempIfPossible(RemovedSection), i);
					Sections.RemoveAt(InsertIndex + 1, EAllowShrinking::No);
					InsertIndex = i;
				}
			}
		}

		return Sections[InsertIndex];
	}
	else
	{
		InsertIndex = Sections.Num();
	}

	if (IsRegistering())
	{
		for (int32 i=0; i < Sections.Num(); ++i)
		{
			if (!Sections[i].bAddedDuringRegister)
			{
				InsertIndex = i;
				break;
			}
		}
	}

	FToolMenuSection& NewSection = Sections.InsertDefaulted_GetRef(InsertIndex);
	NewSection.InitSection(SectionName, InLabel, InPosition);
	NewSection.Owner = UToolMenus::Get()->CurrentOwner();
	NewSection.bIsRegistering = IsRegistering();
	NewSection.bAddedDuringRegister = IsRegistering();
	return NewSection;
}

void UToolMenu::AddSectionScript(
	const FName SectionName,
	const FText& InLabel,
	const FName InsertName,
	const EToolMenuInsertType InsertType,
	EToolMenuSectionAlign Alignment
)
{
	FToolMenuSection& Section = FindOrAddSection(SectionName);

	if (!InLabel.IsEmpty())
	{
		Section.Label = InLabel;
	}

	if (InsertName != NAME_None || InsertType != EToolMenuInsertType::Default)
	{
		Section.InsertPosition = FToolMenuInsert(InsertName, InsertType);
	}

	Section.Alignment = Alignment;
}

void UToolMenu::AddDynamicSectionScript(const FName SectionName, UToolMenuSectionDynamic* InObject)
{
	FToolMenuSection& Section = FindOrAddSection(SectionName);
	Section.ToolMenuSectionDynamic = InObject;
}

void UToolMenu::AddMenuEntryObject(UToolMenuEntryScript* InObject)
{
	FindOrAddSection(InObject->Data.Section).AddEntryObject(InObject);

	if (MenuType == EMultiBoxType::MenuBar || MenuType == EMultiBoxType::ToolBar)
	{
		UToolMenus::Get()->RefreshAllWidgets();
	}
}

void UToolMenu::RemoveMenuEntryObject(UToolMenuEntryScript* InObject)
{
	if (FToolMenuSection* Section = FindSection(InObject->Data.Section))
	{
		Section->RemoveEntryObject(InObject);
	}

	if (MenuType == EMultiBoxType::MenuBar || MenuType == EMultiBoxType::ToolBar)
	{
		UToolMenus::Get()->RefreshAllWidgets();
	}
}

UToolMenu* UToolMenu::AddSubMenuScript(const FName InOwner, const FName SectionName, const FName InName, const FText& InLabel, const FText& InToolTip)
{
	return AddSubMenu(InOwner, SectionName, InName, InLabel, InToolTip);
}

UToolMenu* UToolMenu::AddSubMenu(const FToolMenuOwner InOwner, const FName SectionName, const FName InName, const FText& InLabel, const FText& InToolTip)
{
	FToolMenuEntry Args = FToolMenuEntry::InitSubMenu(InName, InLabel, InToolTip, FNewToolMenuChoice());
	Args.Owner = InOwner;
	FindOrAddSection(SectionName).AddEntry(Args);
	return UToolMenus::Get()->ExtendMenu(*(MenuName.ToString() + TEXT(".") + InName.ToString()));
}

FToolMenuSection* UToolMenu::FindSection(const FName SectionName)
{
	for (FToolMenuSection& Section : Sections)
	{
		if (Section.Name == SectionName)
		{
			return &Section;
		}
	}

	return nullptr;
}

FToolMenuSection& UToolMenu::FindOrAddSection(const FName SectionName)
{
	for (FToolMenuSection& Section : Sections)
	{
		if (Section.Name == SectionName)
		{
			return Section;
		}
	}
	
	return AddSection(SectionName);
}

FToolMenuSection& UToolMenu::FindOrAddSection(
	const FName SectionName,
	const TAttribute<FText>& InLabel,
	const FToolMenuInsert InPosition)
{
	if (FToolMenuSection* FoundSection = FindSection(SectionName))
	{
		return *FoundSection;
	}

	return AddSection(SectionName, InLabel, InPosition);
}

void UToolMenu::RemoveSection(const FName SectionName)
{
	Sections.RemoveAll([SectionName](const FToolMenuSection& Section) { return Section.Name == SectionName; });
}

bool UToolMenu::FindEntry(const FName EntryName, int32& SectionIndex, int32& EntryIndex) const
{
	for (int32 i=0; i < Sections.Num(); ++i)
	{
		EntryIndex = Sections[i].IndexOfBlock(EntryName);
		if (EntryIndex != INDEX_NONE)
		{
			SectionIndex = i;
			return true;
		}
	}

	return false;
}

const FToolMenuEntry* UToolMenu::FindEntry(const FName EntryName) const
{
	for (int32 i=0; i < Sections.Num(); ++i)
	{
		if (const FToolMenuEntry* Found = Sections[i].FindEntry(EntryName))
		{
			return Found;
		}
	}

	return nullptr;
}

FToolMenuEntry* UToolMenu::FindEntry(const FName EntryName)
{
	for (int32 i=0; i < Sections.Num(); ++i)
	{
		if (FToolMenuEntry* Found = Sections[i].FindEntry(EntryName))
		{
			return Found;
		}
	}

	return nullptr;
}

void UToolMenu::AddMenuEntry(const FName SectionName, const FToolMenuEntry& Args)
{
	FindOrAddSection(SectionName).AddEntry(Args);
}

bool UToolMenu::IsEditing() const
{
	return Context.IsEditing();
}

FName UToolMenu::GetSectionName(const FName InEntryName) const
{
	for (const FToolMenuSection& Section : Sections)
	{
		if (Section.IndexOfBlock(InEntryName) != INDEX_NONE)
		{
			return Section.Name;
		}
	}

	return NAME_None;
}

bool UToolMenu::ContainsSection(const FName InName) const
{
	if (InName != NAME_None)
	{
		for (const FToolMenuSection& Section : Sections)
		{
			if (Section.Name == InName)
			{
				return true;
			}
		}
	}

	return false;
}

bool UToolMenu::ContainsEntry(const FName InName) const
{
	if (InName != NAME_None)
	{
		for (const FToolMenuSection& Section : Sections)
		{
			if (Section.FindEntry(InName) != nullptr)
			{
				return true;
			}
		}
	}

	return false;
}

FCustomizedToolMenu* UToolMenu::FindMenuCustomization() const
{
	return UToolMenus::Get()->FindMenuCustomization(MenuName);
}

FCustomizedToolMenu* UToolMenu::AddMenuCustomization() const
{
	return UToolMenus::Get()->AddMenuCustomization(MenuName);
}
	
TArray<FName> UToolMenu::GetMenuHierarchyNames(bool bIncludeSubMenuRoot) const
{
	TArray<FName> HierarchyNames;

	TArray<UToolMenu*> Hierarchy;
	if (UToolMenus::Get()->FindMenu(GetMenuName()) != nullptr)
	{
		Hierarchy = UToolMenus::Get()->CollectHierarchy(GetMenuName());
		for (int32 i = Hierarchy.Num() - 1; i >= 0; --i)
		{
			HierarchyNames.AddUnique(Hierarchy[i]->GetMenuName());
		}
	}

	if (bIncludeSubMenuRoot && SubMenuParent)
	{
		TArray<const UToolMenu*> SubMenuChain = GetSubMenuChain();
		if (SubMenuChain.Num() > 0)
		{
			FString SubMenuFullPath;
			for (int32 i = 1; i < SubMenuChain.Num(); ++i)
			{
				if (SubMenuFullPath.Len() > 0)
				{
					SubMenuFullPath += TEXT(".");
				}
				SubMenuFullPath += SubMenuChain[i]->SubMenuSourceEntryName.ToString();
			}

			// Hierarchy of the initial menu opened in the sub-menu chain of menus
			TArray<UToolMenu*> FirstMenuHierarchy = UToolMenus::Get()->CollectHierarchy(SubMenuChain[0]->GetMenuName());
			for (int32 i = FirstMenuHierarchy.Num() - 1; i >= 0; --i)
			{
				HierarchyNames.AddUnique(UToolMenus::JoinMenuPaths(FirstMenuHierarchy[i]->GetMenuName(), *SubMenuFullPath));
			}
		}
	}	
	Algo::Reverse(HierarchyNames);

	return HierarchyNames;
}

FCustomizedToolMenuHierarchy UToolMenu::GetMenuCustomizationHierarchy() const
{
	FCustomizedToolMenuHierarchy Result;
	
	UToolMenus* ToolMenus = UToolMenus::Get();
	TArray<FName> HierarchyNames = GetMenuHierarchyNames(true);
	for (const FName& ItName : HierarchyNames)
	{
		if (FCustomizedToolMenu* Found = ToolMenus->FindMenuCustomization(ItName))
		{
			Result.Hierarchy.Add(Found);
		}

		if (FCustomizedToolMenu* FoundRuntime = ToolMenus->FindRuntimeMenuCustomization(ItName))
		{
			Result.RuntimeHierarchy.Add(FoundRuntime);
		}
	}

	return Result;
}

FToolMenuProfile* UToolMenu::FindMenuProfile(const FName& ProfileName) const
{
	return UToolMenus::Get()->FindMenuProfile(MenuName, ProfileName);
}

FToolMenuProfile* UToolMenu::AddMenuProfile(const FName& ProfileName) const
{
	return UToolMenus::Get()->AddMenuProfile(MenuName, ProfileName);
}

FToolMenuProfileHierarchy UToolMenu::GetMenuProfileHierarchy(const FName& ProfileName) const
{
	FToolMenuProfileHierarchy Result;
	
	UToolMenus* ToolMenus = UToolMenus::Get();
	TArray<FName> HierarchyNames = GetMenuHierarchyNames(true);
	for (const FName& ItName : HierarchyNames)
	{
		if (FToolMenuProfile* Found = ToolMenus->FindMenuProfile(ItName, ProfileName))
		{
			Result.ProfileHierarchy.Add(Found);
		}

		if (FToolMenuProfile* FoundRuntime = ToolMenus->FindRuntimeMenuProfile(ItName, ProfileName))
		{
			Result.RuntimeProfileHierarchy.Add(FoundRuntime);
		}
	}

	return Result;
}

void UToolMenu::UpdateMenuCustomizationFromMultibox(const TSharedRef<const FMultiBox>& InMultiBox)
{
	FCustomizedToolMenu* Customization = AddMenuCustomization();

	Customization->EntryOrder.Reset();
	Customization->SectionOrder.Reset();

	FName CurrentSectionName = NAME_None;
	const TArray< TSharedRef< const FMultiBlock > >& Blocks = InMultiBox->GetBlocks();
	for (int32 BlockIndex = 0; BlockIndex < Blocks.Num(); ++BlockIndex)
	{
		const TSharedRef< const FMultiBlock >& Block = Blocks[BlockIndex];

		if (Block->GetExtensionHook() == NAME_None)
		{
			continue;
		}

		// Ignore separators that are part of a section heading
		if (Block->IsSeparator() && (BlockIndex + 1 < Blocks.Num()) && Blocks[BlockIndex + 1]->GetType() == EMultiBlockType::Heading)
		{
			continue;
		}

		if (Block->GetType() == EMultiBlockType::Heading)
		{
			CurrentSectionName = Block->GetExtensionHook();
			Customization->SectionOrder.Add(CurrentSectionName);
		}
		else if (CurrentSectionName != NAME_None)
		{
			FCustomizedToolMenuNameArray& EntryOrderForSection = Customization->EntryOrder.FindOrAdd(CurrentSectionName);
			EntryOrderForSection.Names.Add(Block->GetExtensionHook());
		}
	}
}

void UToolMenu::OnMenuDestroyed()
{
	if (bShouldCleanupContextOnDestroy && !SubMenuParent)
	{
		Context.CleanupObjects();
	}

	//Empty();
}

TArray<const UToolMenu*> UToolMenu::GetSubMenuChain() const
{
	TArray<const UToolMenu*> SubMenuChain;

	TSet<const UToolMenu*> SubMenus;
	for (const UToolMenu* CurrentMenu = this; CurrentMenu; CurrentMenu = CurrentMenu->SubMenuParent)
	{
		bool bIsAlreadyInSet = false;
		SubMenus.Add(CurrentMenu, &bIsAlreadyInSet);
		if (bIsAlreadyInSet)
		{
			ensure(!bIsAlreadyInSet);
			break;
		}
		SubMenuChain.Add(CurrentMenu);
	}

	Algo::Reverse(SubMenuChain);

	return SubMenuChain;
}

FString UToolMenu::GetSubMenuNamePath() const
{
	FString SubMenuNamePath;
	TArray<const UToolMenu*> SubMenuChain = GetSubMenuChain();
	if (SubMenuChain.Num() > 0)
	{
		for (int32 i = 1; i < SubMenuChain.Num(); ++i)
		{
			if (SubMenuNamePath.Len() > 0)
			{
				SubMenuNamePath += TEXT(".");
			}
			SubMenuNamePath += SubMenuChain[i]->SubMenuSourceEntryName.ToString();
		}
	}

	return SubMenuNamePath;
}

void UToolMenu::SetExtendersEnabled(bool bEnabled)
{
	bExtendersEnabled = bEnabled;
}

void UToolMenu::Empty()
{
	Context.Empty();
	Sections.Empty();
	SubMenuParent = nullptr;
	ModifyBlockWidgetAfterMake.Unbind();
}

