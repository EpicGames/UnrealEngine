// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "IToolMenusModule.h"

#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolMenuContext)


FToolMenuContext::FToolMenuContext(UObject* InContext)
{
	if (InContext)
	{
		ContextObjects.Add(InContext);
	}
}

FToolMenuContext::FToolMenuContext(UObject* InContext, FContextObjectCleanup&& InCleanup)
{
	if (InContext)
	{
		ContextObjects.Add(InContext);
		if (InCleanup)
		{
			ContextObjectCleanupFuncs.Add(InContext, MoveTemp(InCleanup));
		}
	}
}

FToolMenuContext::FToolMenuContext(TSharedPtr<FUICommandList> InCommandList, TSharedPtr<FExtender> InExtender, UObject* InContext)
{
	if (InContext)
	{
		ContextObjects.Add(InContext);
	}

	if (InExtender.IsValid())
	{
		AddExtender(InExtender);
	}

	AppendCommandList(InCommandList);
}

UObject* FToolMenuContext::FindByClass(UClass* InClass) const
{
	for (UObject* ContextObject : ContextObjects)
	{
		if (ContextObject && ContextObject->IsA(InClass))
		{
			return ContextObject;
		}
	}

	return nullptr;
}

void FToolMenuContext::AppendCommandList(const TSharedRef<FUICommandList>& InCommandList)
{
	const TSharedPtr<FUICommandList> List = InCommandList;
	AppendCommandList(List);
}

void FToolMenuContext::AppendCommandList(const TSharedPtr<FUICommandList>& InCommandList)
{
	if (InCommandList.IsValid())
	{
		CommandLists.Add(InCommandList);

		if (CommandLists.Num() == 1)
		{
			CommandList = InCommandList;
		}
		else if (CommandLists.Num() == 2)
		{
			CommandList = MakeShared<FUICommandList>();
			CommandList->Append(CommandLists[0].ToSharedRef());
			CommandList->Append(InCommandList.ToSharedRef());
		}
		else
		{
			CommandList->Append(InCommandList.ToSharedRef());
		}
	}
}

const FUIAction* FToolMenuContext::GetActionForCommand(TSharedPtr<const FUICommandInfo> Command, TSharedPtr<const FUICommandList>& OutCommandList) const
{
	for (const TSharedPtr<FUICommandList>& CommandListIter : CommandLists)
	{
		if (CommandListIter.IsValid())
		{
			if (const FUIAction* Result = CommandListIter->GetActionForCommand(Command))
			{
				OutCommandList = CommandListIter;
				return Result;
			}
		}
	}

	return nullptr;
}

const FUIAction* FToolMenuContext::GetActionForCommand(TSharedPtr<const FUICommandInfo> Command) const
{
	for (const TSharedPtr<FUICommandList>& CommandListIter : CommandLists)
	{
		if (CommandListIter.IsValid())
		{
			if (const FUIAction* Result = CommandListIter->GetActionForCommand(Command))
			{
				return Result;
			}
		}
	}

	return nullptr;
}

void FToolMenuContext::AddExtender(const TSharedPtr<FExtender>& InExtender)
{
	Extenders.AddUnique(InExtender);
}

TSharedPtr<FExtender> FToolMenuContext::GetAllExtenders() const
{
	return FExtender::Combine(Extenders);
}

void FToolMenuContext::ResetExtenders()
{
	Extenders.Reset();
}

void FToolMenuContext::AppendObjects(const TArray<UObject*>& InObjects)
{
	for (UObject* Object : InObjects)
	{
		AddObject(Object);
	}
}

void FToolMenuContext::AddObject(UObject* InObject)
{
	ContextObjects.AddUnique(InObject);
}

void FToolMenuContext::AddObject(UObject* InObject, FContextObjectCleanup&& InCleanup)
{
	ContextObjects.AddUnique(InObject);
	if (InCleanup)
	{
		ContextObjectCleanupFuncs.Add(InObject, MoveTemp(InCleanup));
	}
}

void FToolMenuContext::AddCleanup(FContextCleanup&& InCleanup)
{
	if (InCleanup)
	{
		ContextCleanupFuncs.Add(MoveTemp(InCleanup));
	}
}

void FToolMenuContext::CleanupObjects()
{
	for (const TTuple<TObjectPtr<UObject>, FContextObjectCleanup>& CleanupPair : ContextObjectCleanupFuncs)
	{
		CleanupPair.Value(CleanupPair.Key);
	}

	for (const FContextCleanup& CleanupFunc : ContextCleanupFuncs)
	{
		CleanupFunc();
	}
}

void FToolMenuContext::Empty()
{
	ContextObjects.Empty();
	ContextObjectCleanupFuncs.Empty();
	ContextCleanupFuncs.Empty();
	CommandLists.Empty();
	CommandList.Reset();
	Extenders.Empty();
}

