// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "IToolMenusModule.h"

#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Input/Events.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolMenuEntry)

FToolMenuVisibilityChoice::FToolMenuVisibilityChoice()
{
}

FToolMenuVisibilityChoice::FToolMenuVisibilityChoice(const FToolMenuVisibilityChoice& Other)
	: Value(Other.Value)
{
}

FToolMenuVisibilityChoice::FToolMenuVisibilityChoice(const TAttribute<EVisibility>& InVisibility)
{
	Value.Set<TAttribute<EVisibility>>(InVisibility);
}

FToolMenuVisibilityChoice::FToolMenuVisibilityChoice(const TAttribute<bool>& InIsVisible)
{
	Value.Set<TAttribute<bool>>(InIsVisible);
}

FToolMenuVisibilityChoice::FToolMenuVisibilityChoice(const FIsActionButtonVisible& InIsActionButtonVisible)
{
	Value.Set<FIsActionButtonVisible>(InIsActionButtonVisible);
}

FToolMenuVisibilityChoice& FToolMenuVisibilityChoice::operator=(TFunction<EVisibility()> VisibilityFunc)
{
	Value.Set<TAttribute<EVisibility>>(TAttribute<EVisibility>::CreateLambda(VisibilityFunc));
	return *this;
}

FToolMenuVisibilityChoice& FToolMenuVisibilityChoice::operator=(TFunction<bool()> IsVisibleFunc)
{
	Value.Set<TAttribute<bool>>(TAttribute<bool>::CreateLambda(IsVisibleFunc));
	return *this;
}

TAttribute<EVisibility> FToolMenuVisibilityChoice::ToVisibilityAttribute() const
{
	if (const TAttribute<EVisibility>* Visibility = Value.TryGet<TAttribute<EVisibility>>())
	{
		return *Visibility;
	}
	
	if (const TAttribute<bool>* IsVisible = Value.TryGet<TAttribute<bool>>())
	{
		return TAttribute<EVisibility>::CreateLambda([IsVisible = *IsVisible]
		{
			return IsVisible.Get() ? EVisibility::Visible : EVisibility::Collapsed;
		});
	}
	
	if (const FIsActionButtonVisible* Action = Value.TryGet<FIsActionButtonVisible>())
	{
		if (Action->IsBound())
		{
			return TAttribute<EVisibility>::CreateLambda([Action = *Action]
			{
				return Action.Execute() ? EVisibility::Visible : EVisibility::Collapsed;
			});
		}
	}
	
	return TAttribute<EVisibility>();
}

bool FToolMenuVisibilityChoice::IsSet() const
{
	if (const TAttribute<EVisibility>* Visibility = Value.TryGet<TAttribute<EVisibility>>())
	{
		return Visibility->IsSet();
	}
	
	if (const TAttribute<bool>* IsVisible = Value.TryGet<TAttribute<bool>>())
	{
		return IsVisible->IsSet();
	}
	
	if (const FIsActionButtonVisible* Action = Value.TryGet<FIsActionButtonVisible>())
	{
		return Action->IsBound();
	}
	
	return false;
}

EVisibility FToolMenuVisibilityChoice::Get() const
{
	if (const TAttribute<EVisibility>* Visibility = Value.TryGet<TAttribute<EVisibility>>())
	{
		return Visibility->Get();
	}
	
	if (const TAttribute<bool>* IsVisible = Value.TryGet<TAttribute<bool>>())
	{
		if (IsVisible->IsSet())
		{
			return IsVisible->Get() ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}
	
	if (const FIsActionButtonVisible* Action = Value.TryGet<FIsActionButtonVisible>())
	{
		if (Action->IsBound())
		{
			return Action->Execute() ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}
	
	return EVisibility::Visible;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FToolMenuEntry::FToolMenuEntry()
	: Type(EMultiBlockType::None)
	, UserInterfaceActionType(EUserInterfaceActionType::Button)
	, bShouldCloseWindowAfterMenuSelection(true)
	, ScriptObject(nullptr)
	, StyleNameOverride(NAME_None)
	, bAddedDuringRegister(false)
	, bCommandIsKeybindOnly(false)
	, ShowInToolbarTopLevel(false)
{
}

FToolMenuEntry::FToolMenuEntry(const FToolMenuOwner InOwner, const FName InName, EMultiBlockType InType)
	: Name(InName)
	, Owner(InOwner)
	, Type(InType)
	, UserInterfaceActionType(EUserInterfaceActionType::Button)
	, bShouldCloseWindowAfterMenuSelection(true)
	, ScriptObject(nullptr)
	, StyleNameOverride(NAME_None)
	, bAddedDuringRegister(false)
	, bCommandIsKeybindOnly(false)
	, ShowInToolbarTopLevel(false)
{
}

FToolMenuEntry::FToolMenuEntry(const FToolMenuEntry&) = default;
FToolMenuEntry::FToolMenuEntry(FToolMenuEntry&&) = default;
FToolMenuEntry& FToolMenuEntry::operator=(const FToolMenuEntry&) = default;
FToolMenuEntry& FToolMenuEntry::operator=(FToolMenuEntry&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

ECheckBoxState FToolMenuEntry::GetCheckState(const FToolMenuContext& InContext) const
{
	{
		TSharedPtr<const FUICommandList> UnusedCommandList;
		if (const FUIAction* UIAction = GetActionForCommand(InContext, UnusedCommandList))
		{
			return UIAction->GetCheckState();
		}
	}

	if (const FUIAction* UIAction = Action.GetUIAction())
	{
		return UIAction->GetCheckState();
	}

	if (const FToolUIAction* ToolUIAction = Action.GetToolUIAction())
	{
		return ToolUIAction->GetActionCheckState.Execute(InContext);
	}

	if (const FToolDynamicUIAction* ToolDynamicUIAction = Action.GetToolDynamicUIAction())
	{
		return ToolDynamicUIAction->GetActionCheckState.Execute(InContext);
	}

	return ECheckBoxState::Undetermined;
}

const FUIAction* FToolMenuEntry::GetActionForCommand(const FToolMenuContext& InContext, TSharedPtr<const FUICommandList>& OutCommandList) const
{
	if (Command.IsValid())
	{
		if (CommandList.IsValid())
		{
			const FUIAction* Result = CommandList->GetActionForCommand(Command);
			if (Result)
			{
				OutCommandList = CommandList;
				return Result;
			}
		}
		else
		{
			return InContext.GetActionForCommand(Command, OutCommandList);
		}
	}

	return nullptr;
}

void FToolMenuEntry::SetCommandList(const TSharedPtr<const FUICommandList>& InCommandList)
{
	CommandList = InCommandList;
}

void FToolMenuEntry::AddKeybindFromCommand(const TSharedPtr< const FUICommandInfo >& InCommand)
{
	if (Type == EMultiBlockType::ToolBarButton)
	{
		Command = InCommand;
		bCommandIsKeybindOnly = true;
	}
	else
	{
		ensureMsgf(false, TEXT("Keybinds from commands can only be associated with toolbar buttons."));
	}
}

void FToolMenuEntry::SetCommand(const TSharedPtr<const FUICommandInfo>& InCommand, TOptional<FName> InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon)
{
	Command = InCommand;
	Name = InName.IsSet() ? InName.GetValue() : InCommand->GetCommandName();
	Label = InLabel.IsSet() ? InLabel : InCommand->GetLabel();
	ToolTip = InToolTip.IsSet() ? InToolTip : InCommand->GetDescription();
	Icon = InIcon.IsSet() ? InIcon : InCommand->GetIcon();
}

FToolMenuEntry FToolMenuEntry::InitMenuEntry(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon, const FToolUIActionChoice& InAction, const EUserInterfaceActionType InUserInterfaceActionType, const FName InTutorialHighlightName, const TOptional<FText>& InInputBindingOverride)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::MenuEntry);
	Entry.TutorialHighlightName = InTutorialHighlightName;
	Entry.Label = InLabel;
	Entry.ToolTip = InToolTip;
	Entry.Icon = InIcon;
	Entry.UserInterfaceActionType = InUserInterfaceActionType;
	Entry.Action = InAction;
	if (InInputBindingOverride.IsSet())
	{
		Entry.InputBindingLabel = InInputBindingOverride.GetValue();
	}
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitMenuEntry(const TSharedPtr< const FUICommandInfo >& InCommand, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon, const FName InTutorialHighlightName, const TOptional<FName> InName)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), NAME_None, EMultiBlockType::MenuEntry);
	Entry.TutorialHighlightName = InTutorialHighlightName;
	Entry.SetCommand(InCommand, InName, InLabel, InToolTip, InIcon);
	Entry.CommandList.Reset();
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitMenuEntry(const FName InNameOverride, const TSharedPtr< const FUICommandInfo >& InCommand, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon, const FName InTutorialHighlightName)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), InNameOverride, EMultiBlockType::MenuEntry);
	Entry.TutorialHighlightName = InTutorialHighlightName;
	Entry.SetCommand(InCommand, InNameOverride, InLabel, InToolTip, InIcon);
	Entry.CommandList.Reset();
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitMenuEntryWithCommandList(const TSharedPtr< const FUICommandInfo >& InCommand, const TSharedPtr< const FUICommandList >& InCommandList, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon, const FName InTutorialHighlightName, const TOptional<FName> InNameOverride)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), NAME_None, EMultiBlockType::MenuEntry);
	Entry.TutorialHighlightName = InTutorialHighlightName;
	Entry.SetCommand(InCommand, InNameOverride, InLabel, InToolTip, InIcon);
	Entry.CommandList = InCommandList;
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitMenuEntry(const FName InName, const FToolUIActionChoice& InAction, const TSharedRef<SWidget>& Widget)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::MenuEntry);
	Entry.Action = InAction;
	Entry.MakeCustomWidget.BindLambda([Widget](const FToolMenuContext&, const FToolMenuCustomWidgetContext&) { return Widget; });
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitDynamicEntry(const FName InName, const FNewToolMenuSectionDelegate& InConstruct)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::MenuEntry);
	Entry.Construct = InConstruct;
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitSubMenu(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewToolMenuChoice& InMakeMenu, const FToolUIActionChoice& InAction, const EUserInterfaceActionType InUserInterfaceActionType, bool bInOpenSubMenuOnClick, const TAttribute<FSlateIcon>& InIcon, const bool bInShouldCloseWindowAfterMenuSelection)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::MenuEntry);
	Entry.Label = InLabel;
	Entry.ToolTip = InToolTip;
	Entry.Icon = InIcon;	
	Entry.Action = InAction;
	Entry.UserInterfaceActionType = InUserInterfaceActionType;
	Entry.bShouldCloseWindowAfterMenuSelection = bInShouldCloseWindowAfterMenuSelection;
	Entry.SubMenuData.bIsSubMenu = true;
	Entry.SubMenuData.ConstructMenu = InMakeMenu;
	Entry.SubMenuData.bOpenSubMenuOnClick = bInOpenSubMenuOnClick;
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitSubMenu(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewToolMenuChoice& InMakeMenu, bool bInOpenSubMenuOnClick, const TAttribute<FSlateIcon>& InIcon, const bool bInShouldCloseWindowAfterMenuSelection, const FName InTutorialHighlightName)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::MenuEntry);
	Entry.TutorialHighlightName = InTutorialHighlightName;
	Entry.Label = InLabel;
	Entry.ToolTip = InToolTip;
	Entry.Icon = InIcon;
	Entry.bShouldCloseWindowAfterMenuSelection = bInShouldCloseWindowAfterMenuSelection;
	Entry.SubMenuData.bIsSubMenu = true;
	Entry.SubMenuData.ConstructMenu = InMakeMenu;
	Entry.SubMenuData.bOpenSubMenuOnClick = bInOpenSubMenuOnClick;
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitSubMenu(const FName InName, const FToolUIActionChoice& InAction, const TSharedRef<SWidget>& InWidget, const FNewToolMenuChoice& InMakeMenu, bool bInShouldCloseWindowAfterMenuSelection)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::MenuEntry);
	Entry.Action = InAction;
	Entry.MakeCustomWidget.BindLambda([=](const FToolMenuContext&, const FToolMenuCustomWidgetContext&) { return InWidget; });
	Entry.bShouldCloseWindowAfterMenuSelection = bInShouldCloseWindowAfterMenuSelection;
	Entry.SubMenuData.bIsSubMenu = true;
	Entry.SubMenuData.ConstructMenu = InMakeMenu;
	Entry.SubMenuData.bOpenSubMenuOnClick = false;
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitToolBarButton(const FName InName, const FToolUIActionChoice& InAction, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon, const EUserInterfaceActionType InUserInterfaceActionType, FName InTutorialHighlightName)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::ToolBarButton);
	Entry.TutorialHighlightName = InTutorialHighlightName;
	Entry.Label = InLabel;
	Entry.ToolTip = InToolTip;
	Entry.Icon = InIcon;
	Entry.UserInterfaceActionType = InUserInterfaceActionType;
	Entry.Action = InAction;
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitToolBarButton(const TSharedPtr< const FUICommandInfo >& InCommand, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon, FName InTutorialHighlightName, const TOptional<FName> InName)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), NAME_None, EMultiBlockType::ToolBarButton);
	Entry.TutorialHighlightName = InTutorialHighlightName;
	Entry.SetCommand(InCommand, InName, InLabel, InToolTip, InIcon);
	return Entry;
}
FToolMenuEntry FToolMenuEntry::InitComboButton(const FName InName, const FToolUIActionChoice& InAction, const FNewToolMenuChoice& InMenuContentGenerator, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon, bool bInSimpleComboBox, FName InTutorialHighlightName)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::ToolBarComboButton);
	Entry.TutorialHighlightName = InTutorialHighlightName;
	Entry.Label = InLabel;
	Entry.ToolTip = InToolTip;
	Entry.Icon = InIcon;
	Entry.Action = InAction;
	Entry.ToolBarData.ComboButtonContextMenuGenerator = InMenuContentGenerator;
	Entry.ToolBarData.bSimpleComboBox = bInSimpleComboBox;
	return Entry;
}

FToolMenuEntry FToolMenuEntry::InitSeparator(const FName InName)
{
	return FToolMenuEntry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::Separator);
}


FToolMenuEntry FToolMenuEntry::InitWidget(const FName InName, const TSharedRef<SWidget>& InWidget, const FText& Label, bool bNoIndent, bool bSearchable, bool bNoPadding, const FText& InToolTipText)
{
	FToolMenuEntry Entry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::Widget);
	Entry.Label = Label;
	Entry.ToolTip = InToolTipText;
	Entry.MakeCustomWidget.BindLambda([=](const FToolMenuContext&, const FToolMenuCustomWidgetContext&) { return InWidget; });
	Entry.WidgetData.bNoIndent = bNoIndent;
	Entry.WidgetData.bSearchable = bSearchable;
	Entry.WidgetData.bNoPadding = bNoPadding;
	return Entry;
}

void FToolMenuEntry::ResetActions()
{
	Action = FToolUIActionChoice();
	Command.Reset();
	CommandList.Reset();
	StringExecuteAction = FToolMenuStringCommand();
	// Note: Cannot reset ScriptObject as it would also remove label and other data
	//ScriptObject = nullptr;
}

bool FToolMenuEntry::IsNonLegacyDynamicConstruct() const
{
	return Construct.IsBound() || IsScriptObjectDynamicConstruct();
}

bool FToolMenuEntry::IsCommandKeybindOnly() const
{
	return bCommandIsKeybindOnly;
}

bool FToolMenuEntry::CommandAcceptsInput(const FKeyEvent& InKeyEvent) const
{
	bool bAccepted = false;
	for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		// check each bound chord
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
		const FInputChord& Chord = *Command->GetActiveChord(ChordIndex);

		bAccepted |= Chord.IsValidChord()
			&& (!Chord.NeedsControl() || InKeyEvent.IsControlDown())
			&& (!Chord.NeedsAlt() || InKeyEvent.IsAltDown())
			&& (!Chord.NeedsShift() || InKeyEvent.IsShiftDown())
			&& (!Chord.NeedsCommand() || InKeyEvent.IsCommandDown())
			&& Chord.Key == InKeyEvent.GetKey();
	}
	return bAccepted;
}

bool FToolMenuEntry::TryExecuteToolUIAction(const FToolMenuContext& InContext)
{
	bool bCanExecute = false;
	if (Action.GetToolUIAction() && Action.GetToolUIAction()->ExecuteAction.IsBound())
	{
		bCanExecute = true;
		if (Action.GetToolUIAction()->CanExecuteAction.IsBound())
		{
			bCanExecute = Action.GetToolUIAction()->CanExecuteAction.Execute(InContext);
		}
		if (bCanExecute)
		{
			Action.GetToolUIAction()->ExecuteAction.Execute(InContext);
		}
	}
	return bCanExecute;
}

void FToolMenuEntry::SetShowInToolbarTopLevel(TAttribute<bool> InTopLevel)
{
	ShowInToolbarTopLevel = InTopLevel;
}

bool FToolMenuEntry::IsScriptObjectDynamicConstruct() const
{
	static const FName ConstructMenuEntryName = GET_FUNCTION_NAME_CHECKED(UToolMenuEntryScript, ConstructMenuEntry);
	return ScriptObject && ScriptObject->GetClass()->IsFunctionImplementedInScript(ConstructMenuEntryName);
}

