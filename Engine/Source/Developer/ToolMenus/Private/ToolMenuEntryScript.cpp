// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenuEntryScript.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "IToolMenusModule.h"

#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolMenuEntryScript)

FScriptSlateIcon::FScriptSlateIcon()
{

}

FScriptSlateIcon::FScriptSlateIcon(const FName InStyleSetName, const FName InStyleName) :
	StyleSetName(InStyleSetName),
	StyleName(InStyleName),
	SmallStyleName(ISlateStyle::Join(InStyleName, ".Small"))
{
}

FScriptSlateIcon::FScriptSlateIcon(const FName InStyleSetName, const FName InStyleName, const FName InSmallStyleName) :
	StyleSetName(InStyleSetName),
	StyleName(InStyleName),
	SmallStyleName(InSmallStyleName)
{
}

FSlateIcon FScriptSlateIcon::GetSlateIcon() const
{
	if (SmallStyleName == NAME_None)
	{
		if (StyleSetName == NAME_None && StyleName == NAME_None)
		{
			return FSlateIcon();
		}

		return FSlateIcon(StyleSetName, StyleName);
	}

	return FSlateIcon(StyleSetName, StyleName, SmallStyleName);
}

UToolMenuEntryScript* UToolMenuEntryScript::GetIfCanSafelyRouteCall(const TWeakObjectPtr<UToolMenuEntryScript>& InWeak)
{
	UToolMenuEntryScript* Object = InWeak.Get();
	return (Object && Object->CanSafelyRouteCall()) ? Object : nullptr;
}

TAttribute<FText> UToolMenuEntryScript::CreateLabelAttribute(FToolMenuContext& Context)
{
	static const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UToolMenuEntryScript, GetLabel);
	if (GetClass()->IsFunctionImplementedInScript(FunctionName))
	{
		TWeakObjectPtr<UToolMenuEntryScript> WeakScriptObject(this);
		TAttribute<FText>::FGetter Getter;
		Getter.BindLambda([WeakScriptObject, Context]()
		{
			UToolMenuEntryScript* Object = GetIfCanSafelyRouteCall(WeakScriptObject);
			return Object ? Object->GetLabel(Context) : FText();
		});

		return TAttribute<FText>::Create(Getter);
	}

	return Data.Label;
}

TAttribute<FText> UToolMenuEntryScript::CreateToolTipAttribute(FToolMenuContext& Context)
{
	static const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UToolMenuEntryScript, GetToolTip);
	if (GetClass()->IsFunctionImplementedInScript(FunctionName))
	{
		TWeakObjectPtr<UToolMenuEntryScript> WeakScriptObject(this);
		TAttribute<FText>::FGetter Getter;
		Getter.BindLambda([WeakScriptObject, Context]()
		{
			UToolMenuEntryScript* Object = GetIfCanSafelyRouteCall(WeakScriptObject);
			return Object ? Object->GetToolTip(Context) : FText();
		});

		return TAttribute<FText>::Create(Getter);
	}

	return Data.ToolTip;
}

TAttribute<FSlateIcon> UToolMenuEntryScript::CreateIconAttribute(FToolMenuContext& Context)
{
	static const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UToolMenuEntryScript, GetIcon);
	if (GetClass()->IsFunctionImplementedInScript(FunctionName))
	{
		TWeakObjectPtr<UToolMenuEntryScript> WeakScriptObject(this);
		TAttribute<FSlateIcon>::FGetter Getter;
		Getter.BindLambda([WeakScriptObject, Context]()
		{
			UToolMenuEntryScript* Object = GetIfCanSafelyRouteCall(WeakScriptObject);
			return Object ? Object->GetSlateIcon(Context) : FSlateIcon();
		});

		return TAttribute<FSlateIcon>::Create(Getter);
	}

	return Data.Icon.GetSlateIcon();
}

TAttribute<bool> UToolMenuEntryScript::CreateShowInToolbarTopLevelAttribute(FToolMenuContext& Context)
{
	static const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UToolMenuEntryScript, ShowInToolbarTopLevel);
	if (GetClass()->IsFunctionImplementedInScript(FunctionName))
	{
		TWeakObjectPtr<UToolMenuEntryScript> WeakScriptObject(this);
		TAttribute<bool>::FGetter Getter;
		Getter.BindLambda(
			[WeakScriptObject, Context]()
			{
				UToolMenuEntryScript* Object = GetIfCanSafelyRouteCall(WeakScriptObject);
				return Object ? Object->ShowInToolbarTopLevel(Context) : false;
			}
		);

		return TAttribute<bool>::Create(Getter);
	}

	return TAttribute<bool>();
}

FSlateIcon UToolMenuEntryScript::GetSlateIcon(const FToolMenuContext& Context) const
{
	return GetIcon(Context).GetSlateIcon();
}

void UToolMenuEntryScript::RegisterMenuEntry()
{
	bHasRegisteredMenuEntry = UToolMenus::AddMenuEntryObject(this);
}

void UToolMenuEntryScript::UnregisterMenuEntry()
{
	if (bHasRegisteredMenuEntry)
	{
		UToolMenus::RemoveMenuEntryObject(this);
		bHasRegisteredMenuEntry = false;
	}
}

void UToolMenuEntryScript::InitEntry(const FName OwnerName, const FName Menu, const FName Section, const FName Name, const FText& Label, const FText& ToolTip)
{
	Data.OwnerName = OwnerName;
	Data.Menu = Menu;
	Data.Section = Section;
	Data.Name = Name;
	Data.Label = Label;
	Data.ToolTip = ToolTip;
}

void UToolMenuEntryScript::ToMenuEntry(FToolMenuEntry& Output)
{
	if (Data.Advanced.bIsSubMenu)
	{
		Output = FToolMenuEntry::InitSubMenu(
			Data.Name,
			Data.Label,
			Data.ToolTip,
			FNewToolMenuChoice(), // Menu will be opened by string: 'Menu' + '.' + 'Name'
			Data.Advanced.bOpenSubMenuOnClick,
			Data.Icon,
			Data.Advanced.bShouldCloseWindowAfterMenuSelection);
	}
	else
	{
		if (Data.Advanced.EntryType == EMultiBlockType::ToolBarButton)
		{
			Output = FToolMenuEntry::InitToolBarButton(
				Data.Name,
				FToolUIActionChoice(), // Action will be handled by 'ScriptObject'
				Data.Label,
				Data.ToolTip,
				Data.Icon,
				Data.Advanced.UserInterfaceActionType,
				Data.Advanced.TutorialHighlight
			);
		}
		else if (Data.Advanced.EntryType == EMultiBlockType::ToolBarComboButton)
		{
			Output = FToolMenuEntry::InitComboButton(
				Data.Name,
				FToolUIActionChoice(), // Action will be handled by 'ScriptObject'
				FNewToolMenuChoice(), // Menu will be opened by string: 'Menu' + '.' + 'Name'
				Data.Label,
				Data.ToolTip,
				Data.Icon,
				Data.Advanced.bSimpleComboBox,
				Data.Advanced.TutorialHighlight
			);
		}
		else
		{
			Output = FToolMenuEntry::InitMenuEntry(Data.Name, Data.Label, Data.ToolTip, Data.Icon, FUIAction());
			Output.UserInterfaceActionType = Data.Advanced.UserInterfaceActionType;
			Output.TutorialHighlightName = Data.Advanced.TutorialHighlight;
		}
	}

	if (!Data.InsertPosition.IsDefault())
	{
		Output.InsertPosition = Data.InsertPosition;
	}

	Output.StyleNameOverride = Data.Advanced.StyleNameOverride;

	Output.ScriptObject = this;

	Output.Owner = Data.OwnerName;
}

FToolMenuEntryScriptDataAdvanced::FToolMenuEntryScriptDataAdvanced() :
	EntryType(EMultiBlockType::MenuEntry),
	UserInterfaceActionType(EUserInterfaceActionType::Button),
	bIsSubMenu(false),
	bOpenSubMenuOnClick(false),
	bShouldCloseWindowAfterMenuSelection(true),
	bSimpleComboBox(false)
{

}

