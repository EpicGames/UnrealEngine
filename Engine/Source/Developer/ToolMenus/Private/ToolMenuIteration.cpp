// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenuIteration.h"

#include "ToolMenus.h"

namespace UE::ToolMenus::Private
{

bool VisitEntriesOfToolMenu(
	UToolMenus* InToolMenus, UToolMenu* InMenu, const FToolMenuContext& InContext, const FToolMenuVisitor& InVisitor
);

bool VisitEntriesOfSubmenu(
	UToolMenus* InToolMenus,
	const UToolMenu* InParentMenu,
	const FName InSubmenuEntryName,
	const FToolMenuContext& InContext,
	const FToolMenuVisitor& InVisitor
)
{
	UToolMenu* Submenu = InToolMenus->GenerateSubMenu(InParentMenu, InSubmenuEntryName);

	return VisitEntriesOfToolMenu(InToolMenus, Submenu, InContext, InVisitor);
}

bool VisitEntriesOfToolMenu(
	UToolMenus* InToolMenus, UToolMenu* InMenu, const FToolMenuContext& InContext, const FToolMenuVisitor& InVisitor
)
{
	for (const FToolMenuSection& Section : InMenu->Sections)
	{
		for (const FToolMenuEntry& Entry : Section.Blocks)
		{
			if (Entry.IsSubMenu())
			{
				const bool bContinue = VisitEntriesOfSubmenu(InToolMenus, InMenu, Entry.Name, InContext, InVisitor);
				if (!bContinue)
				{
					return false;
				}
			}
			else
			{
				FToolMenuIterationInfo IterationInfo(*InMenu, Section, Entry);

				const bool bContinue = InVisitor.Execute(IterationInfo);
				if (!bContinue)
				{
					return false;
				}
			}
		}
	}

	return true;
}

} // namespace UE::ToolMenus::Private

namespace UE::ToolMenus
{

FToolMenuIterationInfo::FToolMenuIterationInfo(
	const UToolMenu& InMenu, const FToolMenuSection& InSection, const FToolMenuEntry& InEntry
)
	: Menu(InMenu)
	, Section(InSection)
	, Entry(InEntry)
{
}

void VisitMenuEntries(
	UToolMenus* InToolMenus, const FName InMenuName, const FToolMenuContext& InContext, const FToolMenuVisitor& InVisitor
)
{
	if (!InToolMenus->FindMenu(InMenuName))
	{
		return;
	}

	// Generate the menu to get the complete menu, including extensions.
	UToolMenu* GeneratedMenu = InToolMenus->GenerateMenu(InMenuName, InContext);

	Private::VisitEntriesOfToolMenu(InToolMenus, GeneratedMenu, InContext, InVisitor);
}

} // namespace UE::ToolMenus
