// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "IToolMenusModule.h"

#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolMenuSection)

FToolMenuSection::FToolMenuSection()
	: ToolMenuSectionDynamic(nullptr)
	, Alignment(EToolMenuSectionAlign::Default)
	, bIsRegistering(false)
	, bAddedDuringRegister(false)
{
}

void FToolMenuSection::InitSection(const FName InName, const TAttribute< FText >& InLabel, const FToolMenuInsert InPosition)
{
	Name = InName;
	Label = InLabel;
	InsertPosition = InPosition;
}

void FToolMenuSection::InitGeneratedSectionCopy(const FToolMenuSection& Source, FToolMenuContext& InContext)
{
	Name = Source.Name;
	Label = Source.Label;
	InsertPosition = Source.InsertPosition;
	Construct = Source.Construct;
	Context = InContext;
	Alignment = Source.Alignment;
	Visibility = Source.Visibility;
	ResizeParams = Source.ResizeParams;
	Sorter = Source.Sorter;
}

bool FToolMenuSection::IsRegistering() const
{
	return bIsRegistering;
}

FToolMenuEntry& FToolMenuSection::AddEntry(const FToolMenuEntry& Args)
{
	if (Args.Name == NAME_None)
	{
		FToolMenuEntry& Result = Blocks.Add_GetRef(Args);
		Result.bAddedDuringRegister = IsRegistering();
		return Result;
	}

	int32 BlockIndex = IndexOfBlock(Args.Name);
	if (BlockIndex != INDEX_NONE)
	{
		Blocks[BlockIndex] = Args;
		Blocks[BlockIndex].bAddedDuringRegister = IsRegistering();
		return Blocks[BlockIndex];
	}
	else
	{
		FToolMenuEntry& Result = Blocks.Add_GetRef(Args);
		Result.bAddedDuringRegister = IsRegistering();
		return Result;
	}
}

FToolMenuEntry& FToolMenuSection::AddEntryObject(UToolMenuEntryScript* InObject)
{
	// Avoid modifying objects that are saved as content on disk
	UToolMenuEntryScript* DestObject = InObject;
	if (DestObject->IsAsset())
	{
		DestObject = DuplicateObject<UToolMenuEntryScript>(InObject, UToolMenus::Get());
	}

	FToolMenuEntry Args;
	DestObject->ToMenuEntry(Args);
	return AddEntry(Args);
}

FToolMenuEntry& FToolMenuSection::AddMenuEntry(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon, const FToolUIActionChoice& InAction, const EUserInterfaceActionType InUserInterfaceActionType, const FName InTutorialHighlightName, const TOptional<FText>& InInputBindingOverride)
{
	return AddEntry(FToolMenuEntry::InitMenuEntry(InName, InLabel, InToolTip, InIcon, InAction, InUserInterfaceActionType, InTutorialHighlightName, InInputBindingOverride));
}

FToolMenuEntry& FToolMenuSection::AddMenuEntry(const TSharedPtr< const FUICommandInfo >& InCommand, const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& InToolTipOverride, const TAttribute<FSlateIcon>& InIconOverride, const FName InTutorialHighlightName, const TOptional<FName> InNameOverride)
{
	check(InCommand.IsValid());

	return AddEntry(FToolMenuEntry::InitMenuEntry(InCommand, InLabelOverride, InToolTipOverride, InIconOverride, InTutorialHighlightName, InNameOverride));
}

FToolMenuEntry& FToolMenuSection::AddMenuEntry(const FName InNameOverride, const TSharedPtr< const FUICommandInfo >& InCommand, const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& InToolTipOverride, const TAttribute<FSlateIcon>& InIconOverride, const FName InTutorialHighlightName)
{
	check(InCommand.IsValid());

	return AddEntry(FToolMenuEntry::InitMenuEntry(InCommand, InLabelOverride, InToolTipOverride, InIconOverride, InTutorialHighlightName, InNameOverride));
}

FToolMenuEntry& FToolMenuSection::AddMenuEntryWithCommandList(const TSharedPtr< const FUICommandInfo >& InCommand, const TSharedPtr< const FUICommandList >& InCommandList, const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& InToolTipOverride, const TAttribute<FSlateIcon>& InIconOverride, const FName InTutorialHighlightName, const TOptional<FName> InNameOverride)
{
	check(InCommand.IsValid());
	check(InCommandList.IsValid());

	return AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(InCommand, InCommandList, InLabelOverride, InToolTipOverride, InIconOverride, InTutorialHighlightName, InNameOverride));
}

FToolMenuEntry& FToolMenuSection::AddDynamicEntry(const FName InName, const FNewToolMenuSectionDelegate& InConstruct)
{
	return AddEntry(FToolMenuEntry::InitDynamicEntry(InName, InConstruct));
}

FToolMenuEntry& FToolMenuSection::AddDynamicEntry(const FName InName, const FNewToolMenuDelegateLegacy& InConstruct)
{
	FToolMenuEntry& Entry = AddEntry(FToolMenuEntry(UToolMenus::Get()->CurrentOwner(), InName, EMultiBlockType::MenuEntry));
	Entry.ConstructLegacy = InConstruct;
	return Entry;
}

FToolMenuEntry& FToolMenuSection::AddSeparator(const FName InName)
{
	FToolMenuEntry& SeparatorEntry = AddEntry(FToolMenuEntry::InitSeparator(InName));

	if (Visibility.IsSet())
	{
		SeparatorEntry.Visibility = Visibility;
	}

	return SeparatorEntry;
}

FToolMenuEntry& FToolMenuSection::AddSubMenu(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewToolMenuChoice& InMakeMenu, const FToolUIActionChoice& InAction, const EUserInterfaceActionType InUserInterfaceActionType, bool bInOpenSubMenuOnClick, const TAttribute<FSlateIcon>& InIcon, const bool bInShouldCloseWindowAfterMenuSelection)
{
	return AddEntry(FToolMenuEntry::InitSubMenu(InName, InLabel, InToolTip, InMakeMenu, InAction, InUserInterfaceActionType, bInOpenSubMenuOnClick, InIcon, bInShouldCloseWindowAfterMenuSelection));
}

FToolMenuEntry& FToolMenuSection::AddSubMenu(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewToolMenuChoice& InMakeMenu, bool bInOpenSubMenuOnClick, const TAttribute<FSlateIcon>& InIcon, const bool bShouldCloseWindowAfterMenuSelection, const FName InTutorialHighlightName)
{
	return AddEntry(FToolMenuEntry::InitSubMenu(InName, InLabel, InToolTip, InMakeMenu, bInOpenSubMenuOnClick, InIcon, bShouldCloseWindowAfterMenuSelection, InTutorialHighlightName));
}

FToolMenuEntry& FToolMenuSection::AddSubMenu(const FName InName, const FToolUIActionChoice& InAction, const TSharedRef<SWidget>& InWidget, const FNewToolMenuChoice& InMakeMenu, bool bShouldCloseWindowAfterMenuSelection)
{
	return AddEntry(FToolMenuEntry::InitSubMenu(InName, InAction, InWidget, InMakeMenu, bShouldCloseWindowAfterMenuSelection));
}

FToolMenuEntry* FToolMenuSection::FindEntry(const FName InName)
{
	for (int32 i=0; i < Blocks.Num(); ++i)
	{
		if (Blocks[i].Name == InName)
		{
			return &Blocks[i];
		}
	}

	return nullptr;
}

const FToolMenuEntry* FToolMenuSection::FindEntry(const FName InName) const
{
	for (int32 i=0; i < Blocks.Num(); ++i)
	{
		if (Blocks[i].Name == InName)
		{
			return &Blocks[i];
		}
	}

	return nullptr;
}

int32 FToolMenuSection::IndexOfBlock(const FName InName) const
{
	for (int32 i=0; i < Blocks.Num(); ++i)
	{
		if (Blocks[i].Name == InName)
		{
			return i;
		}
	}

	return INDEX_NONE;
}

bool FToolMenuSection::IsNonLegacyDynamic() const
{
	return ToolMenuSectionDynamic || Construct.NewToolMenuDelegate.IsBound();
}

int32 FToolMenuSection::RemoveEntry(const FName InName)
{
	return Blocks.RemoveAll([InName](const FToolMenuEntry& Block) { return Block.Name == InName; });
}

int32 FToolMenuSection::RemoveEntryObject(const UToolMenuEntryScript* InObject)
{
	return Blocks.RemoveAll([InObject](const FToolMenuEntry& Block) { return Block.ScriptObject == InObject && Block.Name == InObject->Data.Name; });
}

int32 FToolMenuSection::RemoveEntriesByOwner(const FToolMenuOwner InOwner)
{
	if (InOwner != FToolMenuOwner())
	{
		return Blocks.RemoveAll([InOwner](const FToolMenuEntry& Block) { return Block.Owner == InOwner; });
	}

	return 0;
}

// Note: This function is very similar to UToolMenu::FindInsertIndex.
int32 FToolMenuSection::FindBlockInsertIndex(const FToolMenuEntry& InBlock) const
{
	const FToolMenuInsert InPosition = InBlock.InsertPosition;

	// Insert a Default-positioned entry after all First and Default-positioned entries but before any Last-positioned
	// entries.
	if (InPosition.IsDefault())
	{
		for (int32 i = 0; i < Blocks.Num(); ++i)
		{
			if (Blocks[i].InsertPosition.Position == EToolMenuInsertType::Last)
			{
				return i;
			}
		}

		return Blocks.Num();
	}

	// Insert a First-positioned entry after any other First-positioned entries but before all Default and
	// Last-positioned entries.
	if (InPosition.Position == EToolMenuInsertType::First)
	{
		for (int32 i = 0; i < Blocks.Num(); ++i)
		{
			if (Blocks[i].InsertPosition != InPosition)
			{
				return i;
			}
		}

		return Blocks.Num();
	}

	// Insert a Last-positioned entry after all other entries, include other Last-positioned entries.
	if (InPosition.Position == EToolMenuInsertType::Last)
	{
		for (int32 i = Blocks.Num() - 1; i >= 0; --i)
		{
			if (Blocks[i].InsertPosition.Position == InPosition.Position)
			{
				return i + 1;
			}
		}

		return Blocks.Num();
	}

	int32 DestIndex = IndexOfBlock(InPosition.Name);
	if (DestIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	if (InPosition.Position == EToolMenuInsertType::After)
	{
		++DestIndex;
	}

	for (int32 i = DestIndex; i < Blocks.Num(); ++i)
	{
		if (Blocks[i].InsertPosition != InPosition)
		{
			return i;
		}
	}

	return Blocks.Num();
}

