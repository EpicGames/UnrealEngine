// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenuWidgetCollectionContext.h"

#include "Widgets/SWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolMenuWidgetCollectionContext)

UToolMenuWidgetCollectionContext* UToolMenuWidgetCollectionContext::Get(
	FToolMenuContext& Context,
	bool bCreateIfNeeded /*= true*/)
{
	if (UToolMenuWidgetCollectionContext* ExistingInstance = Context.FindContext<UToolMenuWidgetCollectionContext>())
	{
		return ExistingInstance;
	}

	if (bCreateIfNeeded)
	{
		UToolMenuWidgetCollectionContext* NewInstance = NewObject<UToolMenuWidgetCollectionContext>();
		Context.AddObject(NewInstance);
		return NewInstance;
	}

	return nullptr;
}

void UToolMenuWidgetCollectionContext::AddWidget(const TSharedRef<SWidget>& Widget)
{
	Widgets.AddUnique(Widget);
}

void UToolMenuWidgetCollectionContext::EnumerateWidgets(const TFunctionRef<bool(const TSharedPtr<SWidget>& Widget)>& Callback) const
{
	for (const TWeakPtr<SWidget>& Widget : Widgets)
	{
		if (TSharedPtr<SWidget> PinnedWidget = Widget.Pin())
		{
			if (!Callback(PinnedWidget))
			{
				break;
			}
		}
	}
}

TSharedPtr<SWidget> UToolMenuWidgetCollectionContext::FindWidgetByClassType(const FName& WidgetClassType) const
{
	TSharedPtr<SWidget> FoundWidget;
	EnumerateWidgets([&FoundWidget, &WidgetClassType](const TSharedPtr<SWidget>& Widget)
	{
		if (Widget->GetWidgetClass().GetWidgetType() == WidgetClassType)
		{
			FoundWidget = Widget;
			return false;
		}
		return true;
	});
	return FoundWidget;
}
