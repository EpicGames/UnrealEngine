// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenus.h"
#include "IToolMenusModule.h"
#include "ToolMenusLog.h"

#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Framework/MultiBox/SToolBarComboButtonBlock.h"
#include "Internationalization/Internationalization.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "HAL/PlatformApplicationMisc.h" // For clipboard
#include "Styling/ToolBarStyle.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolMenus)

#define LOCTEXT_NAMESPACE "ToolMenuSubsystem"

DEFINE_LOG_CATEGORY(LogToolMenus);

namespace UE::ToolMenus::Private
{

UToolMenus* CreateToolMenusInstance()
{
	UToolMenus* Instance = NewObject<UToolMenus>();
	Instance->AddToRoot();
	check(Instance);
	return Instance;
}

/** Combine two visibility attributes with the first attribute's non-visibility taking precedence. */
FToolMenuVisibilityChoice CombineVisibility(const FToolMenuVisibilityChoice& PrimaryVisibility, const FToolMenuVisibilityChoice& SecondaryVisibility)
{
	if (PrimaryVisibility.IsSet() && SecondaryVisibility.IsSet())
	{
		return TAttribute<EVisibility>::CreateLambda(
			[Primary = PrimaryVisibility, Secondary = SecondaryVisibility]()
			{
				const EVisibility PrimaryValue = Primary.Get(); 
				if (PrimaryValue != EVisibility::Visible)
				{
					return PrimaryValue;
				}
				return Secondary.Get();
			}
		);
	}
		
	if (PrimaryVisibility.IsSet())
	{
		return PrimaryVisibility;
	}
		
	if (SecondaryVisibility.IsSet())
	{
		return SecondaryVisibility;
	}
		
	return FToolMenuVisibilityChoice();
}

int32 SortSectionAlignment(EToolMenuSectionAlign A, EToolMenuSectionAlign B)
{
	if (A == B)
	{
		return 0;
	}
	
	if (A == EToolMenuSectionAlign::First && B == EToolMenuSectionAlign::Default)
	{
		return -1;
	}
	if (A == EToolMenuSectionAlign::Default && B == EToolMenuSectionAlign::First)
	{
		return 1;
	}
	return A > B ? 1 : -1;
}

} // namespace UE::ToolMenus::Private

namespace UE::ToolMenus
{

FSubBlockReference::FSubBlockReference()
	: ParentMenu(nullptr)
	, Section(nullptr)
	, Entry(nullptr)
{
}

FSubBlockReference::FSubBlockReference(const FSubBlockReference& Other)
	: ParentMenu(Other.ParentMenu)
	, Section(Other.Section)
	, Entry(Other.Entry)
{
}

FSubBlockReference::FSubBlockReference(UToolMenu* InParent, FToolMenuSection* InSection, FToolMenuEntry* InEntry)
	: ParentMenu(InParent)
	, Section(InSection)
	, Entry(InEntry)
{
}


} // namespace UE::ToolMenus

UToolMenus* UToolMenus::Singleton = nullptr;
bool UToolMenus::bHasShutDown = false;
FSimpleMulticastDelegate UToolMenus::StartupCallbacks;
TOptional<FDelegateHandle> UToolMenus::InternalStartupCallbackHandle;

FAutoConsoleCommand ToolMenusRefreshMenuWidget = FAutoConsoleCommand(
	TEXT("ToolMenus.RefreshAllWidgets"),
	TEXT("Refresh All Tool Menu Widgets"),
	FConsoleCommandDelegate::CreateLambda([]() {
		UToolMenus::Get()->RefreshAllWidgets();
	}));

FName FToolMenuStringCommand::GetTypeName() const
{
	static const FName CommandName("Command");
	static const FName PythonName("Python");

	switch (Type)
	{
	case EToolMenuStringCommandType::Command:
		return CommandName;
	case EToolMenuStringCommandType::Python:
		return PythonName;
	case EToolMenuStringCommandType::Custom:
		return CustomType;
	default:
		break;
	}

	return NAME_None;
}

FExecuteAction FToolMenuStringCommand::ToExecuteAction(FName MenuName, const FToolMenuContext& Context) const
{
	if (IsBound())
	{
		return FExecuteAction::CreateStatic(&UToolMenus::ExecuteStringCommand, *this, MenuName, Context);
	}

	return FExecuteAction();
}

FToolUIActionChoice::FToolUIActionChoice(const TSharedPtr< const FUICommandInfo >& InCommand, const FUICommandList& InCommandList)
{
	if (InCommand.IsValid())
	{
		if (const FUIAction* UIAction = InCommandList.GetActionForCommand(InCommand))
		{
			Action = *UIAction;
			ToolAction.Reset();
			DynamicToolAction.Reset();
		}
	}
}

class FPopulateMenuBuilderWithToolMenuEntry
{
public:
	FPopulateMenuBuilderWithToolMenuEntry(FMenuBuilder& InMenuBuilder, UToolMenu* InMenuData, FToolMenuSection& InSection, FToolMenuEntry& InBlock, bool bInAllowSubMenuCollapse) :
		MenuBuilder(InMenuBuilder),
		MenuData(InMenuData),
		Section(InSection),
		Block(InBlock),
		BlockNameOverride(InBlock.Name),
		bAllowSubMenuCollapse(bInAllowSubMenuCollapse),
		bIsEditing(InMenuData->IsEditing())
	{
	}

	void AddSubMenuEntryToMenuBuilder()
	{
		FName SubMenuFullName = UToolMenus::JoinMenuPaths(MenuData->MenuName, BlockNameOverride);
		FNewMenuDelegate NewMenuDelegate;
		bool bSubMenuAdded = false;

		if (Block.SubMenuData.ConstructMenu.NewMenuLegacy.IsBound())
		{
			NewMenuDelegate = Block.SubMenuData.ConstructMenu.NewMenuLegacy;
		}
		else if (Block.SubMenuData.ConstructMenu.NewToolMenuWidget.IsBound() || Block.SubMenuData.ConstructMenu.OnGetContent.IsBound())
		{
			// Full replacement of the widget shown when submenu is opened
			FOnGetContent OnGetContent = UToolMenus::Get()->ConvertWidgetChoice(Block.SubMenuData.ConstructMenu, MenuData->Context, Block);
			if (OnGetContent.IsBound())
			{
				MenuBuilder.AddWrapperSubMenu(
					Block.Label.Get(),
					Block.ToolTip.Get(),
					OnGetContent,
					Block.Icon.Get()
				);
			}
			bSubMenuAdded = true;
		}
		else if (BlockNameOverride == NAME_None)
		{
			if (Block.SubMenuData.ConstructMenu.NewToolMenu.IsBound())
			{
				// Blocks with no name cannot call PopulateSubMenu()
				NewMenuDelegate = FNewMenuDelegate::CreateUObject(UToolMenus::Get(), &UToolMenus::PopulateSubMenuWithoutName, TWeakObjectPtr<UToolMenu>(MenuData), Block);
			}
			else
			{
				UE_LOGF(LogToolMenus, Warning, "Submenu that has no name is missing required delegate: %ls", *MenuData->MenuName.ToString());
			}
		}
		else
		{
			if (Block.SubMenuData.bAutoCollapse && bAllowSubMenuCollapse)
			{
				// Preview the submenu to see if it should be collapsed
				UToolMenu* GeneratedSubMenu = UToolMenus::Get()->GenerateSubMenu(MenuData, BlockNameOverride);
				if (GeneratedSubMenu)
				{
					int32 NumSubMenuEntries = 0;
					FToolMenuEntry* FirstEntry = nullptr;
					for (FToolMenuSection& SubMenuSection : GeneratedSubMenu->Sections)
					{
						NumSubMenuEntries += SubMenuSection.Blocks.Num();
						if (!FirstEntry && SubMenuSection.Blocks.Num() > 0)
						{
							FirstEntry = &SubMenuSection.Blocks[0];
						}
					}

					if (NumSubMenuEntries == 1)
					{
						// Use bAllowSubMenuCollapse = false to avoid recursively collapsing a hierarchy of submenus that each contain one item
						FPopulateMenuBuilderWithToolMenuEntry PopulateMenuBuilderWithToolMenuEntry(MenuBuilder, MenuData, Section, *FirstEntry, /* bAllowSubMenuCollapse= */ false);
						PopulateMenuBuilderWithToolMenuEntry.SetBlockNameOverride(Block.Name);
						PopulateMenuBuilderWithToolMenuEntry.Populate();
						return;
					}
				}
			}

			NewMenuDelegate = FNewMenuDelegate::CreateUObject(UToolMenus::Get(), &UToolMenus::PopulateSubMenu, TWeakObjectPtr<UToolMenu>(MenuData), Block, BlockNameOverride);
		}

		if (!bSubMenuAdded)
		{
			const FToolMenuVisibilityChoice VisibilityOverride = UE::ToolMenus::Private::CombineVisibility(Section.Visibility, Block.Visibility);

			if (Widget.IsValid())
			{
				if (bUIActionIsSet)
				{
					MenuBuilder.AddSubMenu(
						UIAction, Widget.ToSharedRef(), NewMenuDelegate, Block.bShouldCloseWindowAfterMenuSelection, VisibilityOverride
					);
				}
				else
				{
					MenuBuilder.AddSubMenu(
						Widget.ToSharedRef(),
						NewMenuDelegate,
						Block.SubMenuData.bOpenSubMenuOnClick,
						Block.bShouldCloseWindowAfterMenuSelection,
						VisibilityOverride
					);
				}
			}
			else
			{
				if (bUIActionIsSet)
				{
					MenuBuilder.AddSubMenu(
						Block.Label,
						Block.ToolTip,
						NewMenuDelegate,
						UIAction,
						BlockNameOverride,
						Block.UserInterfaceActionType,
						Block.SubMenuData.bOpenSubMenuOnClick,
						Block.Icon.Get(),
						Block.bShouldCloseWindowAfterMenuSelection,
						VisibilityOverride,
						Block.InputBindingLabel
					);
				}
				else
				{
					MenuBuilder.AddSubMenu(
						Block.Label,
						Block.ToolTip,
						NewMenuDelegate,
						Block.SubMenuData.bOpenSubMenuOnClick,
						Block.Icon.Get(),
						Block.bShouldCloseWindowAfterMenuSelection,
						BlockNameOverride,
						Block.TutorialHighlightName,
						VisibilityOverride
					);
				}
			}
		}
	}

	void AddStandardEntryToMenuBuilder()
	{
		const FToolMenuVisibilityChoice VisibilityOverride = UE::ToolMenus::Private::CombineVisibility(Section.Visibility, Block.Visibility);

		// First, check for a ToolUIAction, otherwise do the rest of this (have CommandList and Command)
		// Need another variable to store if we are using a keybind from a command
		if (Block.Command.IsValid())
		{
			bool bPopCommandList = false;
			TSharedPtr<const FUICommandList> CommandListForAction;
			if (Block.GetActionForCommand(MenuData->Context, CommandListForAction) != nullptr && CommandListForAction.IsValid())
			{
				MenuBuilder.PushCommandList(CommandListForAction.ToSharedRef());
				bPopCommandList = true;
			}
			else
			{
				UE_LOGF(LogToolMenus, Error, "UI command not found for menu entry: %ls[%ls], menu: %ls",
					*BlockNameOverride.ToString(), 
					**FTextInspector::GetSourceString(LabelToDisplay.Get()),
					*MenuData->MenuName.ToString());
			}

			MenuBuilder.AddMenuEntry(
				Block.Command, BlockNameOverride, LabelToDisplay, Block.ToolTip, Block.Icon.Get(), NAME_None, VisibilityOverride
			);

			if (bPopCommandList)
			{
				MenuBuilder.PopCommandList();
			}
		}
		else if (Block.ScriptObject)
		{
			UToolMenuEntryScript* ScriptObject = Block.ScriptObject;
			const FSlateIcon Icon = ScriptObject->CreateIconAttribute(MenuData->Context).Get();
			
			FMenuEntryParams MenuEntryParams;
			MenuEntryParams.LabelOverride = ScriptObject->CreateLabelAttribute(MenuData->Context);
			MenuEntryParams.ToolTipOverride = ScriptObject->CreateToolTipAttribute(MenuData->Context);
			MenuEntryParams.IconOverride = Icon;
			MenuEntryParams.DirectActions = UIAction;
			MenuEntryParams.ExtensionHook = ScriptObject->Data.Name;
			MenuEntryParams.UserInterfaceActionType = Block.UserInterfaceActionType;
			MenuEntryParams.TutorialHighlightName = Block.TutorialHighlightName;
			MenuEntryParams.InputBindingOverride = Block.InputBindingLabel;
			MenuEntryParams.Visibility = VisibilityOverride;

			MenuBuilder.AddMenuEntry(MenuEntryParams);
		}
		else
		{
			if (Widget.IsValid())
			{
				FMenuEntryParams MenuEntryParams;
				MenuEntryParams.DirectActions = UIAction;
				MenuEntryParams.EntryWidget = Widget.ToSharedRef();
				MenuEntryParams.ExtensionHook = BlockNameOverride;
				MenuEntryParams.ToolTipOverride = Block.ToolTip;
				MenuEntryParams.UserInterfaceActionType = Block.UserInterfaceActionType;
				MenuEntryParams.TutorialHighlightName = Block.TutorialHighlightName;
				MenuEntryParams.InputBindingOverride = Block.InputBindingLabel;
				MenuEntryParams.Visibility = VisibilityOverride;

				MenuBuilder.AddMenuEntry(MenuEntryParams);
			}
			else
			{
				MenuBuilder.AddMenuEntry(
					LabelToDisplay,
					Block.ToolTip,
					Block.Icon.Get(),
					UIAction,
					BlockNameOverride,
					Block.UserInterfaceActionType,
					Block.TutorialHighlightName,
					Block.InputBindingLabel,
					VisibilityOverride
				);
			}
		}
	}

	void Populate()
	{
		if (Block.ConstructLegacy.IsBound())
		{
			if (!bIsEditing)
			{
				Block.ConstructLegacy.Execute(MenuBuilder, MenuData);
			}

			return;
		}

		const FToolMenuVisibilityChoice VisibilityOverride = UE::ToolMenus::Private::CombineVisibility(Section.Visibility, Block.Visibility);

		UIAction = UToolMenus::ConvertUIAction(Block, MenuData->Context);
		bUIActionIsSet = UIAction.ExecuteAction.IsBound() || UIAction.CanExecuteAction.IsBound() || UIAction.GetActionCheckState.IsBound() || UIAction.IsActionVisibleDelegate.IsBound();

		if (Block.MakeCustomWidget.IsBound())
		{
			FToolMenuCustomWidgetContext EntryWidgetContext;
			TSharedRef<FMultiBox> MultiBox = MenuBuilder.GetMultiBox();
			EntryWidgetContext.StyleSet = MultiBox->GetStyleSet();
			EntryWidgetContext.StyleName = MultiBox->GetStyleName();
			Widget = Block.MakeCustomWidget.Execute(MenuData->Context, EntryWidgetContext);
		}

		LabelToDisplay = Block.Label;
		if (bIsEditing && (!Block.Label.IsSet() || Block.Label.Get().IsEmpty()))
		{
			LabelToDisplay = FText::FromName(BlockNameOverride);
		}

		if (Block.Type == EMultiBlockType::MenuEntry)
		{
			if (Block.IsSubMenu())
			{
				AddSubMenuEntryToMenuBuilder();
			}
			else
			{
				AddStandardEntryToMenuBuilder();
			}
		}
		else if (Block.Type == EMultiBlockType::Separator)
		{
			MenuBuilder.AddSeparator(BlockNameOverride, VisibilityOverride);
		}
		else if (Block.Type == EMultiBlockType::Widget)
		{
			if (bIsEditing)
			{
				FMenuEntryParams MenuEntryParams;
				MenuEntryParams.LabelOverride = LabelToDisplay;
				MenuEntryParams.ToolTipOverride = Block.ToolTip;
				MenuEntryParams.IconOverride = Block.Icon.Get();
				MenuEntryParams.DirectActions = UIAction;
				MenuEntryParams.ExtensionHook = BlockNameOverride;
				MenuEntryParams.UserInterfaceActionType = Block.UserInterfaceActionType;
				MenuEntryParams.TutorialHighlightName = Block.TutorialHighlightName;
				MenuEntryParams.InputBindingOverride = Block.InputBindingLabel;
				MenuEntryParams.Visibility = VisibilityOverride;

				MenuBuilder.AddMenuEntry(MenuEntryParams);
			}
			else
			{
				Block.WidgetData.StyleParams.bNoIndent = Block.WidgetData.bNoIndent;
				MenuBuilder.AddWidget(
					Widget.ToSharedRef(),
					LabelToDisplay.Get(),
					Block.WidgetData.StyleParams,
					Block.WidgetData.ResizeParams,
					Block.WidgetData.bSearchable,
					Block.ToolTip.Get(),
					Block.Icon,
					VisibilityOverride
				);
			}
		}
		else
		{
			UE_LOGF(LogToolMenus, Warning, "Menu '%ls', item '%ls', Menus do not support: %ls", *MenuData->MenuName.ToString(), *BlockNameOverride.ToString(), *UEnum::GetValueAsString(Block.Type));
		}
	};

	void SetBlockNameOverride(const FName InBlockNameOverride) { BlockNameOverride = InBlockNameOverride; };

private:
	FMenuBuilder& MenuBuilder;
	UToolMenu* MenuData;
	FToolMenuSection& Section;
	FToolMenuEntry& Block;
	FName BlockNameOverride;

	FUIAction UIAction;
	TSharedPtr<SWidget> Widget;
	TAttribute<FText> LabelToDisplay;
	bool bAllowSubMenuCollapse;
	bool bUIActionIsSet;
	const bool bIsEditing;
};

UToolMenus::UToolMenus() :
	bNextTickTimerIsSet(false),
	bRefreshWidgetsNextTick(false),
	bCleanupStaleWidgetsNextTick(false),
	bCleanupStaleWidgetsNextTickGC(false),
	bEditMenusMode(false)
{
}

UToolMenus* UToolMenus::Get()
{
	if (!Singleton && !bHasShutDown)
	{
		// Required for StartupModule and ShutdownModule to be called and FindModule to list the ToolsMenus module
		FModuleManager::LoadModuleChecked<IToolMenusModule>("ToolMenus");

		Singleton = UE::ToolMenus::Private::CreateToolMenusInstance();
	}

	return Singleton;
}

void UToolMenus::BeginDestroy()
{
	if (Singleton == this)
	{
		UnregisterPrivateStartupCallback();

		bHasShutDown = true;
		Singleton = nullptr;
	}

	Super::BeginDestroy();
}

bool UToolMenus::IsToolMenuUIEnabled()
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	return !IsRunningCommandlet() && !IsRunningGame() && !IsRunningDedicatedServer() && !IsRunningClientOnly();
}

FName UToolMenus::JoinMenuPaths(const FName Base, const FName Child)
{
	return *(Base.ToString() + TEXT(".") + Child.ToString());
}

bool UToolMenus::SplitMenuPath(const FName MenuPath, FName& OutLeft, FName& OutRight)
{
	if (MenuPath != NAME_None)
	{
		FString Left;
		FString Right;
		if (MenuPath.ToString().Split(TEXT("."), &Left, &Right, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
		{
			OutLeft = *Left;
			OutRight = *Right;
			return true;
		}
	}

	return false;
}

bool UToolMenus::GetDisplayUIExtensionPoints() const
{
	return ShouldDisplayExtensionPoints.IsBound() && ShouldDisplayExtensionPoints.Execute();
}

UToolMenu* UToolMenus::FindMenu(const FName Name)
{
	TObjectPtr<UToolMenu>* Found = Menus.Find(Name);
	return Found ? *Found : nullptr;
}

bool UToolMenus::IsMenuRegistered(const FName Name) const
{
	TObjectPtr<UToolMenu> const * Found = Menus.Find(Name);
	return Found && *Found && (*Found)->IsRegistered();
}

TArray<UToolMenu*> UToolMenus::CollectHierarchy(const FName InName, const TMap<FName, FName>& UnregisteredParentNames)
{
	TArray<UToolMenu*> Result;
	TArray<FName> SubstitutedMenuNames;

	FName CurrentMenuName = InName;
	while (CurrentMenuName != NAME_None)
	{
		FName AdjustedMenuName = CurrentMenuName;
		if (!SubstitutedMenuNames.Contains(AdjustedMenuName))
		{
			if (const FName* SubstitutionName = MenuSubstitutionsDuringGenerate.Find(CurrentMenuName))
			{
				// Allow collection hierarchy when InName is a substitute for one of InName's parents
				// Will occur in menu editor when a substitute menu is selected from drop down list
				bool bSubstituteAlreadyInHierarchy = false;
				for (const UToolMenu* Other : Result)
				{
					if (Other->GetMenuName() == *SubstitutionName)
					{
						bSubstituteAlreadyInHierarchy = true;
						break;
					}
				}

				if (!bSubstituteAlreadyInHierarchy)
				{
					AdjustedMenuName = *SubstitutionName;

					// Handle substitute's parent hierarchy including the original menu again by not substituting the same menu twice
					SubstitutedMenuNames.Add(CurrentMenuName);
				}
			}
		}

		UToolMenu* Current = FindMenu(AdjustedMenuName);
		if (!Current)
		{
			UE_LOGF(LogToolMenus, Warning, "Failed to find menu: %ls for %ls", *AdjustedMenuName.ToString(), *InName.ToString());
			return TArray<UToolMenu*>();
		}

		if (Result.Contains(Current))
		{
			UE_LOGF(LogToolMenus, Warning, "Infinite loop detected in tool menu: %ls", *InName.ToString());
			return TArray<UToolMenu*>();
		}

		Result.Add(Current);

		if (Current->IsRegistered())
		{
			CurrentMenuName = Current->MenuParent;
		}
		else if (const FName* FoundUnregisteredParentName = UnregisteredParentNames.Find(CurrentMenuName))
		{
			CurrentMenuName = *FoundUnregisteredParentName;
		}
		else
		{
			CurrentMenuName = NAME_None;
		}
	}

	Algo::Reverse(Result);

	return Result;
}

void UToolMenus::ApplyStyleToBuilder(FMenuBuilder& InBuilder, const FToolMenuEntryStyle& InStyle)
{
	const ISlateStyle* StyleSet = InBuilder.GetStyleSet();

	// If specified, the provided StyleSet always overrides that specified in the builder
	if (InStyle.StyleSet.IsSet())
	{
		StyleSet = InStyle.StyleSet.GetValue();
	}

	InBuilder.SetStyle(StyleSet, InStyle.StyleName.Get(InBuilder.GetStyleName()));
}

TArray<UToolMenu*> UToolMenus::CollectHierarchy(const FName InName)
{
	TMap<FName, FName> UnregisteredParents;
	return CollectHierarchy(InName, UnregisteredParents);
}

void UToolMenus::ListAllParents(const FName InName, TArray<FName>& AllParents)
{
	for (const UToolMenu* Menu : CollectHierarchy(InName))
	{
		AllParents.Add(Menu->MenuName);
	}
}

void UToolMenus::AssembleMenuSection(UToolMenu* GeneratedMenu, const UToolMenu* Other, FToolMenuSection* DestSection, const FToolMenuSection& OtherSection)
{
	if (!DestSection)
	{
		UE_LOGF(LogToolMenus, Warning, "Trying to add to invalid section for menu: %ls, section: %ls. Default section info will be used instead.", *OtherSection.Owner.TryGetName().ToString(), *OtherSection.Name.ToString());
	}
	// Build list of blocks in expected order including blocks created by construct delegates
	TArray<FToolMenuEntry> RemainingBlocks;
	TArray<FToolMenuEntry> BlocksToAddLast;

	UToolMenu* ConstructedEntries = nullptr;
	for (const FToolMenuEntry& Block : OtherSection.Blocks)
	{
		if (!Block.IsNonLegacyDynamicConstruct())
		{
			if (Block.bAddedDuringRegister)
			{
				RemainingBlocks.Add(Block);
			}
			else
			{
				BlocksToAddLast.Add(Block);
			}
			continue;
		}

		if (ConstructedEntries == nullptr)
		{
			ConstructedEntries = NewToolMenuObject(FName(TEXT("TempAssembleMenuSection")), NAME_None);
			if (!ensure(ConstructedEntries))
			{
				break;
			}
			if (DestSection)
			{
				ConstructedEntries->Context = DestSection->Context;
			}
			else
			{
				ConstructedEntries->Context = FToolMenuContext();
			}
		}

		TArray<FToolMenuEntry> GeneratedEntries;
		GeneratedEntries.Add(Block);

		int32 NumIterations = 0;
		while (GeneratedEntries.Num() > 0)
		{
			FToolMenuEntry& GeneratedEntry = GeneratedEntries[0];
			if (GeneratedEntry.IsNonLegacyDynamicConstruct())
			{
				if (NumIterations++ > 5000)
				{
					FName MenuName = OtherSection.Owner.TryGetName();

					if (Other)
					{
						MenuName = Other->MenuName;
					}
					UE_LOGF(LogToolMenus, Warning, "Possible infinite loop for menu: %ls, section: %ls, block: %ls", *MenuName.ToString(), *OtherSection.Name.ToString(), *Block.Name.ToString());
					break;
				}
				
				ConstructedEntries->Sections.Reset();
				if (GeneratedEntry.IsScriptObjectDynamicConstruct())
				{
					FName SectionName;
					FToolMenuContext SectionContext;
					if (DestSection)
					{
						SectionName = DestSection->Name;
						SectionContext = DestSection->Context;
					}
					GeneratedEntry.ScriptObject->ConstructMenuEntry(ConstructedEntries, SectionName, SectionContext);
				}
				else
				{
					FName SectionName;
					if (DestSection)
					{
						SectionName = DestSection->Name;
					}
					FToolMenuSection& ConstructedSection = ConstructedEntries->AddSection(SectionName);
					ConstructedSection.Context = ConstructedEntries->Context;
					GeneratedEntry.Construct.Execute(ConstructedSection);
				}
				GeneratedEntries.RemoveAt(0, EAllowShrinking::No);

				// Combine all user's choice of selections here into the current section target
				// If the user wants to add items to different sections they will need to create dynamic section instead (for now)
				int32 NumBlocksInserted = 0;
				for (FToolMenuSection& ConstructedSection : ConstructedEntries->Sections)
				{
					for (FToolMenuEntry& ConstructedBlock : ConstructedSection.Blocks)
					{
						if (ConstructedBlock.InsertPosition.IsDefault())
						{
							ConstructedBlock.InsertPosition = Block.InsertPosition;
						}
					}
					GeneratedEntries.Insert(ConstructedSection.Blocks, NumBlocksInserted);
					NumBlocksInserted += ConstructedSection.Blocks.Num();
				}
			}
			else
			{
				if (Block.bAddedDuringRegister)
				{
					RemainingBlocks.Add(GeneratedEntry);
				}
				else
				{
					BlocksToAddLast.Add(GeneratedEntry);
				}
				GeneratedEntries.RemoveAt(0, EAllowShrinking::No);
			}
		}
	}

	if (ConstructedEntries)
	{
		ConstructedEntries->Empty();
		ConstructedEntries = nullptr;
	}

	RemainingBlocks.Append(BlocksToAddLast);

	// Only do this loop if there is a section to insert into. We need to early-out here or it will be an infinite loop
	if (DestSection)
	{
		// Repeatedly loop because insert location may not exist until later in list
		while (RemainingBlocks.Num() > 0)
		{
			int32 NumHandled = 0;
			for (int32 i = 0; i < RemainingBlocks.Num(); ++i)
			{
				FToolMenuEntry& Block = RemainingBlocks[i];
				
				if (!Block.Name.IsNone())
				{
					// Replace blocks that have the same name. Children blocks override parent blocks.
					int32 TargetIndex = DestSection->IndexOfBlock(Block.Name);
					if (TargetIndex != INDEX_NONE)
					{
						if (Block.InsertPosition.IsDefault())
						{
							DestSection->Blocks[TargetIndex] = Block;
                            RemainingBlocks.RemoveAt(i);
                            --i;
                            ++NumHandled;
                            break;
						}
						else
						{
							// Allow positioning logic to run as specified on the new item
							DestSection->Blocks.RemoveAt(TargetIndex);
						}
					}
				}
				
				int32 DestIndex = DestSection->FindBlockInsertIndex(Block);
				if (DestIndex != INDEX_NONE)
				{
					DestSection->Blocks.Insert(Block, DestIndex);
					RemainingBlocks.RemoveAt(i);
					--i;
					++NumHandled;
					// Restart loop because items earlier in the list may need to attach to this block
					break;
				}
			}
			if (NumHandled == 0)
			{
				FToolMenuEntry& Block = RemainingBlocks[0];
				if ((Block.InsertPosition.Fallback & EToolMenuInsertFallback::Log) == EToolMenuInsertFallback::Log)
				{
					UE_LOGF(LogToolMenus, Warning, "Menu item not found: '%ls' for insert: '%ls'", *Block.InsertPosition.Name.ToString(), *Block.Name.ToString());
				}
				if ((Block.InsertPosition.Fallback & EToolMenuInsertFallback::Insert) == EToolMenuInsertFallback::Insert)
				{
					// Insert the first remaining block and try	again. Later blocks may have depended on that first block.
					DestSection->Blocks.Add(Block);
					RemainingBlocks.RemoveAt(0);
				}
				else
				{
					break;	
				}
			}
		}
	}
}

void UToolMenus::AssembleMenu(UToolMenu* GeneratedMenu, const UToolMenu* Other)
{
	TArray<FToolMenuSection> RemainingSections;

	UToolMenu* ConstructedSections = nullptr;
	for (const FToolMenuSection& OtherSection : Other->Sections)
	{
		if (!OtherSection.IsNonLegacyDynamic())
		{
			RemainingSections.Add(OtherSection);
			continue;
		}
		
		if (ConstructedSections == nullptr)
		{
			ConstructedSections = NewToolMenuObject(FName(TEXT("TempAssembleMenu")), NAME_None);
			if (!ensure(ConstructedSections))
			{
				break;
			}
			ConstructedSections->Context = GeneratedMenu->Context;
			ConstructedSections->MenuType = GeneratedMenu->MenuType;
		}

		TArray<FToolMenuSection> GeneratedSections;
		GeneratedSections.Add(OtherSection);

		int32 NumIterations = 0;
		while (GeneratedSections.Num() > 0)
		{
			if (GeneratedSections[0].IsNonLegacyDynamic())
			{
				if (NumIterations++ > 5000)
				{
					UE_LOGF(LogToolMenus, Warning, "Possible infinite loop for menu: %ls, section: %ls", *Other->MenuName.ToString(), *OtherSection.Name.ToString());
					break;
				}

				ConstructedSections->Sections.Reset();
				
				if (GeneratedSections[0].ToolMenuSectionDynamic)
				{
					GeneratedSections[0].ToolMenuSectionDynamic->ConstructSections(ConstructedSections, GeneratedMenu->Context);
				}
				else if (GeneratedSections[0].Construct.NewToolMenuDelegate.IsBound())
				{
					GeneratedSections[0].Construct.NewToolMenuDelegate.Execute(ConstructedSections);
				}

				for (FToolMenuSection& ConstructedSection : ConstructedSections->Sections)
				{
					if (ConstructedSection.InsertPosition.IsDefault())
					{
						ConstructedSection.InsertPosition = GeneratedSections[0].InsertPosition;
					}
				}
				
				GeneratedSections.RemoveAt(0, EAllowShrinking::No);
				GeneratedSections.Insert(ConstructedSections->Sections, 0);
			}
			else
			{
				RemainingSections.Add(GeneratedSections[0]);
				GeneratedSections.RemoveAt(0, EAllowShrinking::No);
			}
		}
	}

	if (ConstructedSections)
	{
		ConstructedSections->Empty();
		ConstructedSections = nullptr;
	}

	while (RemainingSections.Num() > 0)
	{
		int32 NumHandled = 0;
		for (int32 i=0; i < RemainingSections.Num(); ++i)
		{
			FToolMenuSection& RemainingSection = RemainingSections[i];

			// Menubars do not have sections, combine all sections into one
			if (GeneratedMenu->MenuType == EMultiBoxType::MenuBar)
			{
				RemainingSection.Name = NAME_None;
			}

			// Update existing section
			FToolMenuSection* Section = GeneratedMenu->FindSection(RemainingSection.Name);
			if (!Section)
			{
				// Try add new section (if insert location exists)
				int32 DestIndex = GeneratedMenu->FindInsertIndex(RemainingSection);
				if (DestIndex != INDEX_NONE)
				{
					GeneratedMenu->Sections.InsertDefaulted(DestIndex);
					Section = &GeneratedMenu->Sections[DestIndex];
					Section->InitGeneratedSectionCopy(RemainingSection, GeneratedMenu->Context);
				}
				else
				{
					continue;
				}
			}
			else
			{
				// Allow overriding label
				if (!Section->Label.IsSet() && RemainingSection.Label.IsSet())
				{
					Section->Label = RemainingSection.Label;
				}

				// Let child menu override dynamic legacy section
				if (!RemainingSection.IsNonLegacyDynamic())
				{
					Section->Construct = RemainingSection.Construct;
				}

				if (!Section->Visibility.IsSet() && RemainingSection.Visibility.IsSet())
				{
					Section->Visibility = RemainingSection.Visibility;
				}
			}

			AssembleMenuSection(GeneratedMenu, Other, Section, RemainingSection);
			RemainingSections.RemoveAt(i);
			--i;
			++NumHandled;
			break;
		}
		if (NumHandled == 0)
		{
			for (const FToolMenuSection& RemainingSection : RemainingSections)
			{
				UE_LOGF(LogToolMenus, Warning, "Menu section not found: '%ls' for insert: '%ls'", *RemainingSection.InsertPosition.Name.ToString(), *RemainingSection.Name.ToString());
			}
			break;
		}
	}
}

bool UToolMenus::GetEditMenusMode() const
{
	return bEditMenusMode;
}

void UToolMenus::SetEditMenusMode(bool bShow)
{
	if (bEditMenusMode != bShow)
	{
		bEditMenusMode = bShow;
		RefreshAllWidgets();
	}
}

void UToolMenus::RemoveCustomization(const FName InName)
{
	int32 FoundIndex = FindMenuCustomizationIndex(InName);
	if (FoundIndex != INDEX_NONE)
	{
		CustomizedMenus.RemoveAt(FoundIndex, EAllowShrinking::No);
	}
}

int32 UToolMenus::FindMenuCustomizationIndex(const FName InName)
{
	for (int32 i = 0; i < CustomizedMenus.Num(); ++i)
	{
		if (CustomizedMenus[i].Name == InName)
		{
			return i;
		}
	}

	return INDEX_NONE;
}

FCustomizedToolMenu* UToolMenus::FindMenuCustomization(const FName InName)
{
	for (int32 i = 0; i < CustomizedMenus.Num(); ++i)
	{
		if (CustomizedMenus[i].Name == InName)
		{
			return &CustomizedMenus[i];
		}
	}

	return nullptr;
}

FCustomizedToolMenu* UToolMenus::AddMenuCustomization(const FName InName)
{
	if (FCustomizedToolMenu* Found = FindMenuCustomization(InName))
	{
		return Found;
	}
	else
	{
		FCustomizedToolMenu& NewCustomization = CustomizedMenus.AddDefaulted_GetRef();
		NewCustomization.Name = InName;
		return &NewCustomization;
	}
}

FCustomizedToolMenu* UToolMenus::FindRuntimeMenuCustomization(const FName InName)
{
	for (int32 i = 0; i < RuntimeCustomizedMenus.Num(); ++i)
	{
		if (RuntimeCustomizedMenus[i].Name == InName)
		{
			return &RuntimeCustomizedMenus[i];
		}
	}

	return nullptr;
}

FCustomizedToolMenu* UToolMenus::AddRuntimeMenuCustomization(const FName InName)
{
	if (FCustomizedToolMenu* Found = FindRuntimeMenuCustomization(InName))
	{
		return Found;
	}
	else
	{
		FCustomizedToolMenu& NewCustomization = RuntimeCustomizedMenus.AddDefaulted_GetRef();
		NewCustomization.Name = InName;
		return &NewCustomization;
	}
}

FToolMenuProfile* UToolMenus::FindMenuProfile(const FName InMenuName, const FName InProfileName)
{
	if(FToolMenuProfileMap* FoundMenu = MenuProfiles.Find(InMenuName))
	{
		return FoundMenu->MenuProfiles.Find(InProfileName);
	}

	return nullptr;
}

FToolMenuProfile* UToolMenus::AddMenuProfile(const FName InMenuName, const FName InProfileName)
{
	if (FToolMenuProfile* Found = FindMenuProfile(InMenuName, InProfileName))
	{
		return Found;
	}
	else
	{
		FToolMenuProfileMap& FoundMenu = MenuProfiles.FindOrAdd(InMenuName);
		
		FToolMenuProfile& NewCustomization = FoundMenu.MenuProfiles.Add(InProfileName, FToolMenuProfile());
		NewCustomization.Name = InProfileName;
		return &NewCustomization;
	}
}


FToolMenuProfile* UToolMenus::FindRuntimeMenuProfile(const FName InMenuName, const FName InProfileName)
{
	if(FToolMenuProfileMap* FoundMenu = RuntimeMenuProfiles.Find(InMenuName))
	{
		return FoundMenu->MenuProfiles.Find(InProfileName);
	}

	return nullptr;
}

FToolMenuProfile* UToolMenus::AddRuntimeMenuProfile(const FName InMenuName, const FName InProfileName)
{
	if (FToolMenuProfile* Found = FindRuntimeMenuProfile(InMenuName, InProfileName))
	{
		return Found;
	}
	else
	{
		FToolMenuProfileMap& FoundMenu = RuntimeMenuProfiles.FindOrAdd(InMenuName);
		
		FToolMenuProfile& NewCustomization = FoundMenu.MenuProfiles.Add(InProfileName, FToolMenuProfile());
		NewCustomization.Name = InProfileName;
		return &NewCustomization;
	}
}

void UToolMenus::ApplyCustomizationAndProfiles(UToolMenu* GeneratedMenu)
{
	// Apply all profiles that are active by looking for them in the context
	UToolMenuProfileContext* ProfileContext = GeneratedMenu->FindContext<UToolMenuProfileContext>();
	
	if(ProfileContext)
	{
		for(const FName& ActiveProfile : ProfileContext->ActiveProfiles)
		{
			FToolMenuProfileHierarchy MenuProfileHieararchy = GeneratedMenu->GetMenuProfileHierarchy(ActiveProfile);

			if (MenuProfileHieararchy.ProfileHierarchy.Num() != 0 || MenuProfileHieararchy.RuntimeProfileHierarchy.Num() != 0)
			{
				FToolMenuProfile MenuProfile = MenuProfileHieararchy.GenerateFlattenedMenuProfile();
				ApplyProfile(GeneratedMenu, MenuProfile);
			}
			else
			{
				UE_LOGF(LogToolMenus, Verbose, "Menu Profile %ls for menu %ls not found!", *ActiveProfile.ToString(), *GeneratedMenu->GetMenuName().ToString());

			}
		}
	}
	
	// Apply the customization for the menu (if any)
	FCustomizedToolMenuHierarchy CustomizationHierarchy = GeneratedMenu->GetMenuCustomizationHierarchy();
	if (CustomizationHierarchy.Hierarchy.Num() != 0 || CustomizationHierarchy.RuntimeHierarchy.Num() != 0)
	{
		FCustomizedToolMenu CustomizedMenu = CustomizationHierarchy.GenerateFlattened();
		ApplyCustomization(GeneratedMenu, CustomizedMenu);
	}
}

void UToolMenus::ApplyProfile(UToolMenu* GeneratedMenu, const FToolMenuProfile& MenuProfile)
{
	if (MenuProfile.IsSuppressExtenders())
	{
		GeneratedMenu->SetExtendersEnabled(false);
	}
	
	TArray<FToolMenuSection> NewSections(GeneratedMenu->Sections);
	
	// Hide items based on deny list
	if (MenuProfile.MenuPermissions.HasFiltering())
	{
		for (int32 SectionIndex = 0; SectionIndex < NewSections.Num(); ++SectionIndex)
		{
			FToolMenuSection& Section = NewSections[SectionIndex];
			for (int32 i = 0; i < Section.Blocks.Num(); ++i)
			{
				if (!MenuProfile.MenuPermissions.PassesFilter(Section.Blocks[i].Name))
				{
					Section.Blocks.RemoveAt(i);
					--i;
				}
			}
		}
	}

	// Hide sections and entries
	if (!GeneratedMenu->IsEditing())
	{
		for (int32 SectionIndex = 0; SectionIndex < NewSections.Num(); ++SectionIndex)
		{
			FToolMenuSection& Section = NewSections[SectionIndex];
			if (MenuProfile.IsSectionHidden(Section.Name))
			{
				NewSections.RemoveAt(SectionIndex);
				--SectionIndex;
				continue;
			}

			for (int32 i = 0; i < Section.Blocks.Num(); ++i)
			{
				if (MenuProfile.IsEntryHidden(Section.Blocks[i].Name))
				{
					Section.Blocks.RemoveAt(i);
					--i;
				}
			}
		}
	}

	GeneratedMenu->Sections = NewSections;

}

void UToolMenus::ApplyCustomization(UToolMenu* GeneratedMenu, const FCustomizedToolMenu& CustomizedMenu)
{
	TArray<FToolMenuSection> NewSections;
	NewSections.Reserve(GeneratedMenu->Sections.Num());

	TSet<FName> PlacedEntries;

	TArray<int32> NewSectionIndices;
	NewSectionIndices.Reserve(GeneratedMenu->Sections.Num());

	// Add sections with customized ordering first
	for (const FName& SectionName : CustomizedMenu.SectionOrder)
	{
		if (SectionName != NAME_None)
		{
			int32 OriginalIndex = GeneratedMenu->Sections.IndexOfByPredicate([SectionName](const FToolMenuSection& OriginalSection) { return OriginalSection.Name == SectionName; });
			if (OriginalIndex != INDEX_NONE)
			{
				NewSectionIndices.Add(OriginalIndex);
			}
		}
	}

	// Remaining sections get added to the end
	for (int32 i = 0; i < GeneratedMenu->Sections.Num(); ++i)
	{
		NewSectionIndices.AddUnique(i);
	}

	// Copy sections
	for (int32 i = 0; i < NewSectionIndices.Num(); ++i)
	{
		FToolMenuSection& NewSection = NewSections.Add_GetRef(GeneratedMenu->Sections[NewSectionIndices[i]]);
		NewSection.Blocks.Reset();
	}

	// Add entries placed by customization
	for (int32 i = 0; i < NewSectionIndices.Num(); ++i)
	{
		const FToolMenuSection& OriginalSection = GeneratedMenu->Sections[NewSectionIndices[i]];
		FToolMenuSection& NewSection = NewSections[i];

		if (OriginalSection.Name != NAME_None)
		{
			if (const FCustomizedToolMenuNameArray* EntryOrder = CustomizedMenu.EntryOrder.Find(OriginalSection.Name))
			{
				for (const FName& EntryName : EntryOrder->Names)
				{
					if (EntryName != NAME_None)
					{
						if (FToolMenuEntry* SourceEntry = GeneratedMenu->FindEntry(EntryName))
						{
							NewSection.Blocks.Add(*SourceEntry);
							PlacedEntries.Add(EntryName);
						}
					}
				}
			}
		}
	}

	// Handle entries not placed by customization
	for (int32 i = 0; i < NewSectionIndices.Num(); ++i)
	{
		const FToolMenuSection& OriginalSection = GeneratedMenu->Sections[NewSectionIndices[i]];
		FToolMenuSection& NewSection = NewSections[i];

		for (const FToolMenuEntry& OriginalEntry : OriginalSection.Blocks)
		{
			if (OriginalEntry.Name == NAME_None)
			{
				NewSection.Blocks.Add(OriginalEntry);
			}
			else
			{
				bool bAlreadyPlaced = false;
				PlacedEntries.Add(OriginalEntry.Name, &bAlreadyPlaced);
				if (!bAlreadyPlaced)
				{
					NewSection.Blocks.Add(OriginalEntry);
				}
			}
		}
	}

	GeneratedMenu->Sections = NewSections;

	ApplyProfile(GeneratedMenu, CustomizedMenu);
}

void UToolMenus::AssembleMenuHierarchy(UToolMenu* GeneratedMenu, const TArray<UToolMenu*>& Hierarchy)
{
	TGuardValue<bool> SuppressRefreshWidgetsRequestsGuard(bSuppressRefreshWidgetsRequests, true);

	for (const UToolMenu* FoundParent : Hierarchy)
	{
		AssembleMenu(GeneratedMenu, FoundParent);
	}

	for (FToolMenuSection& Section : GeneratedMenu->Sections)
	{
		if (Section.Sorter.IsBound())
		{
			Section.Blocks.StableSort([&Section](const FToolMenuEntry& A, const FToolMenuEntry& B)
			{
				return Section.Sorter.Execute(A, B, Section.Context);
			});
		}
	}
	
	ApplyCustomizationAndProfiles(GeneratedMenu);
}

UToolMenu* UToolMenus::GenerateSubMenu(const UToolMenu* InGeneratedParent, const FName InBlockName)
{
	if (InGeneratedParent == nullptr || InBlockName == NAME_None)
	{
		return nullptr;
	}

	FName SubMenuFullName = JoinMenuPaths(InGeneratedParent->GetMenuName(), InBlockName);

	const FToolMenuEntry* Block = InGeneratedParent->FindEntry(InBlockName);
	if (!Block)
	{
		return nullptr;
	}

	TGuardValue<bool> SuppressRefreshWidgetsRequestsGuard(bSuppressRefreshWidgetsRequests, true);

	// Submenus that are constructed by delegates can also be overridden by menus in the database
	TArray<UToolMenu*> Hierarchy;
	{
		struct FMenuHierarchyInfo
		{
			FMenuHierarchyInfo() : BaseMenu(nullptr), SubMenu(nullptr) { }
			FName BaseMenuName;
			FName SubMenuName;
			UToolMenu* BaseMenu;
			UToolMenu* SubMenu;
		};

		TArray<FMenuHierarchyInfo> HierarchyInfos;
		TArray<UToolMenu*> UnregisteredHierarchy;

		// Walk up all parent menus trying to find a menu
		FName BaseName = InGeneratedParent->GetMenuName();
		while (BaseName != NAME_None)
		{
			FMenuHierarchyInfo& Info = HierarchyInfos.AddDefaulted_GetRef();
			Info.BaseMenuName = BaseName;
			Info.BaseMenu = FindMenu(Info.BaseMenuName);
			Info.SubMenuName = JoinMenuPaths(BaseName, InBlockName);
			Info.SubMenu = FindMenu(Info.SubMenuName);

			if (Info.SubMenu)
			{
				if (Info.SubMenu->IsRegistered())
				{
					if (UnregisteredHierarchy.Num() == 0)
					{
						Hierarchy = CollectHierarchy(Info.SubMenuName);
					}
					else
					{
						UnregisteredHierarchy.Add(Info.SubMenu);
					}
					break;
				}
				else
				{
					UnregisteredHierarchy.Add(Info.SubMenu);
				}
			}

			BaseName = Info.BaseMenu  ? Info.BaseMenu->MenuParent : NAME_None;
		}

		if (UnregisteredHierarchy.Num() > 0)
		{
			// Create lookup for UToolMenus that were extended but not registered
			TMap<FName, FName> UnregisteredParentNames;
			for (int32 i = 0; i < UnregisteredHierarchy.Num() - 1; ++i)
			{
				UnregisteredParentNames.Add(UnregisteredHierarchy[i]->GetMenuName(), UnregisteredHierarchy[i + 1]->GetMenuName());
			}
			Hierarchy = CollectHierarchy(UnregisteredHierarchy[0]->GetMenuName(), UnregisteredParentNames);
		}
	}

	// Construct menu using delegate and insert as root so it can be overridden
	TArray<UToolMenu*> MenusToCleanup;
	if (Block->SubMenuData.ConstructMenu.NewToolMenu.IsBound())
	{
		UToolMenu* Menu = NewToolMenuObject(FName(TEXT("TempGenerateSubMenu")), SubMenuFullName);
		MenusToCleanup.Add(Menu);
		Menu->Context = InGeneratedParent->Context;

		// Submenu specific data
		Menu->SubMenuParent = InGeneratedParent;
		Menu->SubMenuSourceEntryName = InBlockName;

		Block->SubMenuData.ConstructMenu.NewToolMenu.Execute(Menu);
		Menu->MenuName = SubMenuFullName;
		Hierarchy.Insert(Menu, 0);
	}

	// Populate menu builder with final menu
	if (Hierarchy.Num() > 0)
	{
		UToolMenu* GeneratedMenu = NewToolMenuObject(FName(TEXT("GeneratedSubMenu")), SubMenuFullName);
		GeneratedMenu->InitGeneratedCopy(Hierarchy[0], SubMenuFullName, &InGeneratedParent->Context);
		for (UToolMenu* HiearchyItem : Hierarchy)
		{
			if (HiearchyItem && !HiearchyItem->bExtendersEnabled)
			{
				GeneratedMenu->SetExtendersEnabled(false);
				break;
			}
		}
		GeneratedMenu->SubMenuParent = InGeneratedParent;
		GeneratedMenu->SubMenuSourceEntryName = InBlockName;
		AssembleMenuHierarchy(GeneratedMenu, Hierarchy);
		for (UToolMenu* MenuToCleanup : MenusToCleanup)
		{
			MenuToCleanup->Empty();
		}
		MenusToCleanup.Empty();
		return GeneratedMenu;
	}

	for (UToolMenu* MenuToCleanup : MenusToCleanup)
	{
		MenuToCleanup->Empty();
	}
	MenusToCleanup.Empty();

	return nullptr;
}

void UToolMenus::PopulateSubMenu(FMenuBuilder& MenuBuilder, TWeakObjectPtr<UToolMenu> InParent, const FToolMenuEntry InEntry, const FName InBlockName)
{
	if (UToolMenu* GeneratedMenu = GenerateSubMenu(InParent.Get(), InBlockName))
	{
		MenuBuilder.SetExtendersEnabled(GeneratedMenu->bExtendersEnabled);

		// Apply Style override
		ApplyStyleToBuilder(MenuBuilder, InEntry.SubMenuData.Style);

		PopulateMenuBuilder(MenuBuilder, GeneratedMenu);
	}
}

void UToolMenus::PopulateSubMenuWithoutName(FMenuBuilder& MenuBuilder, TWeakObjectPtr<UToolMenu> InParent, const FToolMenuEntry InEntry)
{
	const UToolMenu* InGeneratedParent = InParent.Get();
	if (InGeneratedParent == nullptr)
	{
		return;
	}

	if (InEntry.SubMenuData.ConstructMenu.NewToolMenu.IsBound())
	{
		UToolMenu* Menu = NewToolMenuObject(FName(TEXT("SubMenuWithoutName")), NAME_None); // Menu does not have a name
		Menu->Context = InGeneratedParent->Context;

		// Submenu specific data
		Menu->SubMenuParent = InGeneratedParent;
		Menu->SubMenuSourceEntryName = NAME_None; // Entry does not have a name

		// Apply Style override
		ApplyStyleToBuilder(MenuBuilder, InEntry.SubMenuData.Style);

		InEntry.SubMenuData.ConstructMenu.NewToolMenu.Execute(Menu);
		Menu->MenuName = NAME_None; // Menu does not have a name

		PopulateMenuBuilder(MenuBuilder, Menu);
	}
}

TSharedRef<SWidget> UToolMenus::GenerateToolbarComboButtonMenu(TWeakObjectPtr<UToolMenu> InParent, const FName InBlockName)
{
	if (UToolMenu* GeneratedMenu = GenerateSubMenu(InParent.Get(), InBlockName))
	{
		return GenerateWidget(GeneratedMenu);
	}

	return SNullWidget::NullWidget;
}

void UToolMenus::PopulateMenuBuilder(FMenuBuilder& MenuBuilder, UToolMenu* MenuData)
{
	MenuBuilder.SetSearchable(MenuData->bSearchable);

	const bool bIsEditing = MenuData->IsEditing();
	if (GetEditMenusMode() && !bIsEditing && EditMenuDelegate.IsBound())
	{
		TWeakObjectPtr<UToolMenu> WeakMenuPtr = MenuData;
		const FName MenuName = MenuData->GetMenuName();
		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("EditMenu_Label", "Edit Menu: {0}"), FText::FromName(MenuName)),
			LOCTEXT("EditMenu_ToolTip", "Open menu editor"),
			EditMenuIcon,
			FExecuteAction::CreateLambda([MenuName, WeakMenuPtr]()
			{
				FPlatformApplicationMisc::ClipboardCopy(*MenuName.ToString());
				if (UToolMenu* InMenu = WeakMenuPtr.Get())
				{
					UToolMenus::Get()->EditMenuDelegate.ExecuteIfBound(InMenu);
				}
			}),
			"MenuName"
		);
	}

	for (int i=0; i < MenuData->Sections.Num(); ++i)
	{
		FToolMenuSection& Section = MenuData->Sections[i];
		if (Section.Construct.NewToolMenuDelegateLegacy.IsBound())
		{
			if (!bIsEditing)
			{
				Section.Construct.NewToolMenuDelegateLegacy.Execute(MenuBuilder, MenuData);
			}
			continue;
		}

		if (bIsEditing)
		{
			// Always provide label when editing so we have area to drag/drop and hide sections
			FText LabelText = Section.Label.Get();
			if (LabelText.IsEmpty())
			{
				LabelText = FText::FromName(Section.Name);
			}
			MenuBuilder.BeginSection(Section.Name, LabelText, Section.Visibility, Section.ResizeParams);
		}
		else
		{
			MenuBuilder.BeginSection(Section.Name, Section.Label, Section.Visibility, Section.ResizeParams);
		}

		for (FToolMenuEntry& Block : Section.Blocks)
		{
			FPopulateMenuBuilderWithToolMenuEntry PopulateMenuBuilderWithToolMenuEntry(MenuBuilder, MenuData, Section, Block, /* bAllowSubMenuCollapse= */ true);
			PopulateMenuBuilderWithToolMenuEntry.Populate();
		}

		MenuBuilder.EndSection();
	}

	MenuBuilder.GetMultiBox()->WeakToolMenu = MenuData;
	AddReferencedContextObjects(MenuBuilder.GetMultiBox(), MenuData);
}

void UToolMenus::ExtractChildBlocksFromSubMenu(UToolMenu* ParentMenu, FToolMenuEntry& InBlock, TArray<UE::ToolMenus::FSubBlockReference>& SubMenuBlocks)
{
	const bool bBuiltViaDelegate =
		ConvertWidgetChoice(InBlock.ToolBarData.ComboButtonContextMenuGenerator, ParentMenu->Context, InBlock).IsBound();

	const bool bHasChildren = (InBlock.Type == EMultiBlockType::ToolBarComboButton
								  || (InBlock.Type == EMultiBlockType::MenuEntry && InBlock.IsSubMenu()))
						   && !bBuiltViaDelegate;

	if (!bHasChildren)
	{
		return;
	}

	UToolMenu* const SubMenu = GenerateSubMenu(ParentMenu, InBlock.Name);
	if (!SubMenu)
	{
		return;
	}

	// Add the blocks reversed to follow a depth-first iteration.
	for (int i = SubMenu->Sections.Num() - 1; i >= 0; --i)
	{
		FToolMenuSection& Section = SubMenu->Sections[i];

		for (int j = Section.Blocks.Num() - 1; j >= 0; --j)
		{
			FToolMenuEntry& Block = Section.Blocks[j];
			SubMenuBlocks.Push(UE::ToolMenus::FSubBlockReference(SubMenu, &Section, &Block));
		}
	}
}

void UToolMenus::PopulateToolBarBuilderWithTopLevelChildren(FToolBarBuilder& ToolBarBuilder, UToolMenu* ParentMenu, FToolMenuEntry& InBlock, bool bAddSpaceAfterRaisedChildren)
{
	using namespace UE::ToolMenus;

	TArray<FSubBlockReference> SubMenuBlocks;
	// Seed the blocks with the passed-in submenu.
	ExtractChildBlocksFromSubMenu(ParentMenu, InBlock, SubMenuBlocks);

	// Collect the blocks that might be raised to the top level. Also include separators so we can visualize those in
	// the toolbar when they appear between two raised blocks.
	TArray<FSubBlockReference> BlocksToAdd;

	// Traverse the sub menu blocks we've found thus far to find more grandchild blocks that are raised (raised set to
	// boolean true) or could be dynamically raised (raised set to a TAttribute<bool> driven by a delegate) to the
	// top-level toolbar.
	int32 NumIterations = 0;
	while (SubMenuBlocks.Num() > 0)
	{
		const FSubBlockReference SubMenuBlock = SubMenuBlocks.Pop(EAllowShrinking::No);
		UToolMenu* const SubMenu = SubMenuBlock.ParentMenu;
		FToolMenuEntry* const Block = SubMenuBlock.Entry;

		// Keep track of how many blocks we've visited to ensure we don't loop indefinitely.
		if (++NumIterations > 5000)
		{
			UE_LOGF(LogToolMenus, Warning,
				"Possible infinite loop for menu with section menu. parent menu: %ls, menu: %ls, block: %ls",
				*ParentMenu->MenuName.ToString(), *SubMenu->MenuName.ToString(), *Block->Name.ToString());
			break;
		}

		const TAttribute<bool> ScriptShowInToolbarTopLevel = Block->ScriptObject
			? Block->ScriptObject->CreateShowInToolbarTopLevelAttribute(ParentMenu->Context)
			: TAttribute<bool>();

		const bool bIsBound = Block->ShowInToolbarTopLevel.IsBound() || ScriptShowInToolbarTopLevel.IsBound();
		const bool bIsSetToValue = !bIsBound && (Block->ShowInToolbarTopLevel.IsSet() || ScriptShowInToolbarTopLevel.IsSet());
		const bool bIsSetToTrueValue = bIsSetToValue && (Block->ShowInToolbarTopLevel.Get() || ScriptShowInToolbarTopLevel.Get());
		if (bIsBound || bIsSetToTrueValue || Block->Type == EMultiBlockType::Separator)
		{
			BlocksToAdd.Add(SubMenuBlock);
		}

		ExtractChildBlocksFromSubMenu(SubMenu, *Block, SubMenuBlocks);
	}

	// Do not allow leading separators.
	while (BlocksToAdd.Num() > 0 && BlocksToAdd[0].Entry->Type == EMultiBlockType::Separator)
	{
		BlocksToAdd.RemoveAt(0);
	}

	// Do not allow trailing separators.
	while (BlocksToAdd.Num() > 0 && BlocksToAdd[BlocksToAdd.Num() - 1].Entry->Type == EMultiBlockType::Separator)
	{
		BlocksToAdd.RemoveAt(BlocksToAdd.Num() - 1);
	}

	// Do not allow rows of separators.
	for (int i = 1; i < BlocksToAdd.Num(); ++i)
	{
		const bool bIsCurrentSeparator = BlocksToAdd[i].Entry->Type == EMultiBlockType::Separator;
		const bool bWasPreviousSeparator = BlocksToAdd[i - 1].Entry->Type == EMultiBlockType::Separator;
		const bool bPartOfRowOfSeparators = bIsCurrentSeparator && bWasPreviousSeparator;

		if (bPartOfRowOfSeparators)
		{
			BlocksToAdd.RemoveAt(i--);
		}
	}

	if (BlocksToAdd.IsEmpty())
	{
		return;
	}

	// Dynamic visibility of trailing raised-children spacer.
	TArray<TAttribute<EVisibility>> AllRaisedVisibilities = TArray<TAttribute<EVisibility>>();

	// Dynamic visibility of separators
	//
	// We add separators in the top-level toolbar between raised entries if the raised entries lived in different
	// sections or if a separator was explicitly added between the raised entries.
	//
	// Since entries can be dynamically raised, added toolbar separators must have dynamic visiblity. To support this,
	// we record the visibility delegates of previously raised entries so separator visiblity delegates can use them.
	//
	// A menu might look like this:
	//
	//  |-- previous1 -|               |-- previous2 -|               |-- previous3 -|             |---- next ----|
	//  raisedA raisedB SEPARATOR(N-2) raisedC raisedD SEPARATOR(N-1) raisedE raisedF SEPARATOR(N) raisedG raisedH
	//
	// PreviousEntries = (raisedA, raisedB, raisedC, raisedD, raisedE, raisedF)
	// NextEntries = (raisedG, raisedH)
	//
	// Separator visibility is then calculated like this:
	//
	//   sep_vis = anyVisible(nextEntries) && anyVisible(PreviousEntries
	//

	TArray<TAttribute<EVisibility>> PreviousVisibilities;
	// This has to be heap allocated so we can still add do it after a separator visibility delegate captures it.
	TSharedPtr<TArray<TAttribute<EVisibility>>> NextVisibilities = MakeShared<TArray<TAttribute<EVisibility>>>();

	bool bHasRaisedEntrySinceLastSeparator = false;
	// Seed the previous section with the first block's section so we don't start with adding a separator because
	// sections seem to have changed.
	FName PreviousSectionName = BlocksToAdd[0].Section->Name;
	FName PreviousBlockGroupName = NAME_None;
	for (int i = 0; i < BlocksToAdd.Num(); ++i)
	{
		UToolMenu* const SubMenu = BlocksToAdd[i].ParentMenu;
		FToolMenuSection* const Section = BlocksToAdd[i].Section;
		FToolMenuEntry* const Entry = BlocksToAdd[i].Entry;

		// Add a separator if one was found or a new section was encountered.
		if (bHasRaisedEntrySinceLastSeparator
			&& (Section->Name != PreviousSectionName || Entry->Type == EMultiBlockType::Separator))
		{
			// Step entry visibility delegate records forward now that we encountered a new separator.
			PreviousVisibilities.Append(*NextVisibilities);
			NextVisibilities = MakeShared<TArray<TAttribute<EVisibility>>>();

			const TAttribute<EVisibility> VisibilityOverride = TAttribute<EVisibility>::CreateLambda(
				[Previous = PreviousVisibilities, Next = NextVisibilities]()
				{
					// This function calculates this expression and earlies out if possible.
					// const bool bVisible = bAnyNext && bAnyPrevious;

					bool bAnyNext = false;
					for (const TAttribute<EVisibility>& Visibility : *Next)
					{
						if (Visibility.Get() == EVisibility::Visible)
						{
							bAnyNext = true;
							break;
						}
					}

					if (!bAnyNext)
					{
						return EVisibility::Collapsed;
					}

					for (const TAttribute<EVisibility>& Visibility : Previous)
					{
						if (Visibility.Get() == EVisibility::Visible)
						{
							return EVisibility::Visible;
						}
					}

					return EVisibility::Collapsed;
				}
			);

			const FName UnsetExtensionHook = NAME_None;

			FMenuEntryResizeParams ResizeParams;
			ResizeParams.VisibleInOverflow = false;

			ToolBarBuilder.AddSeparator(UnsetExtensionHook, VisibilityOverride, ResizeParams);
			bHasRaisedEntrySinceLastSeparator = false;
		}

		// Make sure we actually add the entry if the reason we added a separator above was that the section names changed.
		if (Entry->Type != EMultiBlockType::Separator)
		{
			const FName EntryBlockGroupName = Entry->ToolBarData.BlockGroupName;
			if (EntryBlockGroupName != PreviousBlockGroupName)
			{
				if (!PreviousBlockGroupName.IsNone())
				{
					ToolBarBuilder.EndBlockGroup();
				}
				if (!EntryBlockGroupName.IsNone())
				{
					ToolBarBuilder.BeginBlockGroup();
				}
			}
			PreviousBlockGroupName = EntryBlockGroupName;
		
			constexpr bool bRaiseToTopLevel = true;
			PopulateToolBarBuilderWithEntry(ToolBarBuilder, SubMenu, *Section, *Entry, bRaiseToTopLevel);
			bHasRaisedEntrySinceLastSeparator = true;
			
			constexpr bool bEmbedActionOrCommand = true;
			const TAttribute<EVisibility> EntryVisibility = CalculateToolbarVisibility(SubMenu, *Section, *Entry, bRaiseToTopLevel, bEmbedActionOrCommand);

			// Keep track of added entries' visibilities so separators can set their visibility override.
			NextVisibilities->Add(EntryVisibility);

			if (bAddSpaceAfterRaisedChildren)
			{
				AllRaisedVisibilities.Add(EntryVisibility);
			}
		}

		PreviousSectionName = Section->Name;
	}
	
	if (!PreviousBlockGroupName.IsNone())
	{
		ToolBarBuilder.EndBlockGroup();
	}

	if (bAddSpaceAfterRaisedChildren)
	{
		const FMenuEntryStyleParams StyleParams;
		const FName TutorialHighlightName = NAME_None;
		const bool bSearchable = false;
		const FNewMenuDelegate CustomMenuDelegate;

		TAttribute<EVisibility> SpacerVisibilityOverride = TAttribute<EVisibility>::CreateLambda(
			[AllRaisedVisibilities]()
			{
				for (const TAttribute<EVisibility>& Visibility : AllRaisedVisibilities)
				{
					if (Visibility.Get() == EVisibility::Visible)
					{
						return EVisibility::Visible;
					}
				}

				return EVisibility::Collapsed;
			}
		);

		FMenuEntryResizeParams ResizeParams;
		// Never show spacers in overflow menus.
		ResizeParams.VisibleInOverflow = false;
		// Default clipping priority is 0. Set to a negative number so that the spaces drop before any content.
		ResizeParams.ClippingPriority = -100;

		const FName StyleName = InBlock.StyleNameOverride.IsNone() ? ParentMenu->StyleName : InBlock.StyleNameOverride;
		const FToolBarStyle ToolbarStyle = ParentMenu->GetStyleSet()->GetWidgetStyle<FToolBarStyle>(StyleName);
		ToolBarBuilder.AddWidget(
			SNew(SSpacer).Size(FVector2D(ToolbarStyle.RaisedChildrenRightPadding, 0)),
			StyleParams,
			TutorialHighlightName,
			bSearchable,
			CustomMenuDelegate,
			SpacerVisibilityOverride,
			ResizeParams
		);
	}
}

void UToolMenus::PopulateToolBarBuilderWithEntry(
	FToolBarBuilder& ToolBarBuilder,
	UToolMenu* MenuData,
	FToolMenuSection& Section,
	FToolMenuEntry& Block,
	bool bIsRaisingToTopLevel,
	bool bIsLastBlockOfLastSection
)
{
	if (Block.ToolBarData.ConstructLegacy.IsBound())
	{
		Block.ToolBarData.ConstructLegacy.Execute(ToolBarBuilder, MenuData);
		return;
	}

	// Override the style name.
	{
		FName OverrideStyleName = Block.ToolBarData.StyleNameOverride;
		if (OverrideStyleName == NAME_None)
		{
			OverrideStyleName = Block.StyleNameOverride;
		}

		// Add the .Raised suffix for menu entries raised to the top level.
		if (bIsRaisingToTopLevel)
		{
			if (OverrideStyleName != NAME_None)
			{
				OverrideStyleName = ISlateStyle::Join(OverrideStyleName, ".Raised");
			}
			else
			{
				// We have to search up the submenu parent chain here because the immediate menu we're a part of might
				// not have a style set while a parent could.
				const UToolMenu* CurrentMenu = MenuData;
				FName MenuStyleName = NAME_None;
				while (MenuStyleName == NAME_None && CurrentMenu->SubMenuParent)
				{
					CurrentMenu = CurrentMenu->SubMenuParent.Get();
					MenuStyleName = CurrentMenu->StyleName;
				}

				if (MenuStyleName != NAME_None)
				{
					OverrideStyleName = ISlateStyle::Join(MenuStyleName, ".Raised");
				}
			}
		}

		ToolBarBuilder.BeginStyleOverride(OverrideStyleName);
	}

	constexpr bool bEmbedActionOrCommand = false;
	const TAttribute<EVisibility> Visibility = CalculateToolbarVisibility(MenuData, Section, Block, bIsRaisingToTopLevel, bEmbedActionOrCommand);

	const FUIAction UIAction = [&Block, MenuData]()
		{
			if (Block.ToolBarData.ActionOverride.IsSet())
			{
				return UToolMenus::ConvertUIAction(Block.ToolBarData.ActionOverride.GetValue(), MenuData->Context);
			}
			else
			{
				return UToolMenus::ConvertUIAction(Block, MenuData->Context);
			}
		}();

	TAttribute<FText> ToolbarLabelOverride;
	if (Block.ToolBarData.LabelOverride.IsSet())
	{
		ToolbarLabelOverride = Block.ToolBarData.LabelOverride;
	}
	else if (const bool bHasIcon = Block.Icon.IsSet() || (Block.Command.IsValid() && Block.Command->GetIcon().IsSet());
			 bHasIcon && bIsRaisingToTopLevel)
	{
		// Set the toolbar label to the empty string if we're raising an entry that has an icon. This makes
		// raising/pinning of icons less annoying because the intended design is for them to not have a label. We can
		// still use the ToolbarLabelOverride to bypass this.
		ToolbarLabelOverride = FText();
	}

	FMenuEntryResizeParams ActualResizeParams = Block.ToolBarData.ResizeParams;
	if (bIsRaisingToTopLevel && !Block.ToolBarData.ResizeParams.VisibleInOverflow.IsSet())
	{
		ActualResizeParams.VisibleInOverflow = false;
	}

	if (Block.Type == EMultiBlockType::ToolBarButton || (Block.Type == EMultiBlockType::MenuEntry && !Block.IsSubMenu()))
	{
		if (Block.Command.IsValid() && !Block.IsCommandKeybindOnly())
		{
			bool bPopCommandList = false;
			TSharedPtr<const FUICommandList> CommandListForAction;
			if (Block.GetActionForCommand(MenuData->Context, CommandListForAction) != nullptr
				&& CommandListForAction.IsValid())
			{
				ToolBarBuilder.PushCommandList(CommandListForAction.ToSharedRef());
				bPopCommandList = true;
			}
			else
			{
				UE_LOGF(LogToolMenus, Verbose, "UI command not found for toolbar entry: %ls, toolbar: %ls",
					*Block.Name.ToString(), *MenuData->MenuName.ToString());
			}

			TSharedPtr<FToolBarButtonBlock> ButtonBlock = ToolBarBuilder.AddToolBarButton(
				Block.Command,
				Block.Name,
				Block.Label,
				Block.ToolTip,
				Block.Icon,
				Block.TutorialHighlightName,
				FNewMenuDelegate(),
				Visibility,
				ToolbarLabelOverride,
				ActualResizeParams
			);
			
			if (Block.MakeCustomWidget.IsBound())
			{
				FToolMenuCustomWidgetContext EntryWidgetContext;
				TSharedRef<FMultiBox> MultiBox = ToolBarBuilder.GetMultiBox();
				EntryWidgetContext.StyleSet = MultiBox->GetStyleSet();
				EntryWidgetContext.StyleName = MultiBox->GetStyleName();
				ButtonBlock->SetCustomWidget(Block.MakeCustomWidget.Execute(MenuData->Context, EntryWidgetContext));
			}

			if (bPopCommandList)
			{
				ToolBarBuilder.PopCommandList();
			}
		}
		else if (Block.ScriptObject)
		{
			UToolMenuEntryScript* ScriptObject = Block.ScriptObject;
			const TAttribute<FSlateIcon> Icon = ScriptObject->CreateIconAttribute(MenuData->Context);

			ToolBarBuilder.AddToolBarButton(
				UIAction,
				ScriptObject->Data.Name,
				ScriptObject->CreateLabelAttribute(MenuData->Context),
				ScriptObject->CreateToolTipAttribute(MenuData->Context),
				Icon,
				Block.UserInterfaceActionType,
				Block.TutorialHighlightName,
				Visibility,
				ToolbarLabelOverride,
				ActualResizeParams
			);
		}
		else
		{
			TSharedPtr<FToolBarButtonBlock> ButtonBlock = ToolBarBuilder.AddToolBarButton(
				UIAction,
				Block.Name,
				Block.Label,
				Block.ToolTip,
				Block.Icon,
				Block.UserInterfaceActionType,
				Block.TutorialHighlightName,
				Visibility,
				ToolbarLabelOverride,
				ActualResizeParams
			);
			
			if (Block.MakeCustomWidget.IsBound())
			{
				FToolMenuCustomWidgetContext EntryWidgetContext;
				TSharedRef<FMultiBox> MultiBox = ToolBarBuilder.GetMultiBox();
				EntryWidgetContext.StyleSet = MultiBox->GetStyleSet();
				EntryWidgetContext.StyleName = MultiBox->GetStyleName();
				ButtonBlock->SetCustomWidget(Block.MakeCustomWidget.Execute(MenuData->Context, EntryWidgetContext));
			}
		}

		if (Block.ToolBarData.OptionsDropdownData.IsValid())
		{
			FOnGetContent OnGetContent = ConvertWidgetChoice(
				Block.ToolBarData.OptionsDropdownData->MenuContentGenerator, MenuData->Context, Block);
			ToolBarBuilder.AddComboButton(
				Block.ToolBarData.OptionsDropdownData->Action,
				OnGetContent,
				Block.Label,
				Block.ToolBarData.OptionsDropdownData->ToolTip,
				Block.Icon,
				true,
				Block.TutorialHighlightName,
				Visibility,
				ToolbarLabelOverride,
				Block.ToolBarData.PlacementOverride,
				Block.UserInterfaceActionType,
				ActualResizeParams
			);
		}
	}
	else if (Block.Type == EMultiBlockType::ToolBarComboButton || (Block.Type == EMultiBlockType::MenuEntry && Block.IsSubMenu()))
	{
		bool bCouldHaveChildren = false;
		FOnGetContent OnGetContent = ConvertWidgetChoice(Block.ToolBarData.ComboButtonContextMenuGenerator, MenuData->Context, Block);
		
		// Allow non-tool-menu-creating choices to be applied
		if (!OnGetContent.IsBound() && !Block.SubMenuData.ConstructMenu.NewToolMenu.IsBound())
		{
			OnGetContent = ConvertWidgetChoice(Block.SubMenuData.ConstructMenu, MenuData->Context, Block);
		}
		
		if (!OnGetContent.IsBound())
		{
			// Handle tool-menu-generating lambdas.
			// Make sure to keep a strong reference to this submenu so it stays around until it is opened. This is
			// needed here because we're could be a submenu of a submenu, so not even our parent is added to the Menus
			// map and therefore our parent could have been gargage collected before this submenu is opened.
			OnGetContent = FOnGetContent::CreateLambda(
				[this, StrongMenuData = TStrongObjectPtr<UToolMenu>(MenuData), BlockName = Block.Name]() -> TSharedRef<SWidget>
				{
					return this->GenerateToolbarComboButtonMenu(StrongMenuData.Get(), BlockName);
				}
			);
			bCouldHaveChildren = true;
		}
		
		TSharedPtr<FToolBarComboButtonBlock> ComboButtonBlock = ToolBarBuilder.AddComboButton(
			UIAction,
			OnGetContent,
			Block.Label,
			Block.ToolTip,
			Block.Icon,
			Block.ToolBarData.bSimpleComboBox,
			Block.TutorialHighlightName,
			Visibility,
			ToolbarLabelOverride,
			Block.ToolBarData.PlacementOverride,
			Block.UserInterfaceActionType,
			ActualResizeParams
		);
			
		if (ComboButtonBlock.IsValid() && Block.MakeCustomWidget.IsBound())
		{
			TSharedRef<FMultiBox> MultiBox = ToolBarBuilder.GetMultiBox();

			FToolMenuCustomWidgetContext EntryWidgetContext;
			EntryWidgetContext.StyleSet = MultiBox->GetStyleSet();
			EntryWidgetContext.StyleName = MultiBox->GetStyleName();

			TSharedPtr<SWidget> CustomWidget = Block.MakeCustomWidget.Execute(MenuData->Context, EntryWidgetContext);
			if (CustomWidget.IsValid())
			{
				ComboButtonBlock->SetCustomWidget(CustomWidget);
			}
		}

		// Also add any top-level flagged children to the toolbar.
		if (bCouldHaveChildren && !bIsRaisingToTopLevel)
		{
			PopulateToolBarBuilderWithTopLevelChildren(ToolBarBuilder, MenuData, Block, !bIsLastBlockOfLastSection);
		}
	}
	else if (Block.Type == EMultiBlockType::Separator)
	{
		ToolBarBuilder.AddSeparator(Block.Name);
	}
	else if (Block.Type == EMultiBlockType::Widget)
	{
		TSharedPtr<SWidget> Widget;

		if (Block.MakeCustomWidget.IsBound())
		{
			FToolMenuCustomWidgetContext EntryWidgetContext;
			TSharedRef<FMultiBox> MultiBox = ToolBarBuilder.GetMultiBox();
			EntryWidgetContext.StyleSet = MultiBox->GetStyleSet();
			EntryWidgetContext.StyleName = MultiBox->GetStyleName();
			Widget = Block.MakeCustomWidget.Execute(MenuData->Context, EntryWidgetContext);
		}

		FMenuEntryStyleParams StyleParams = Block.WidgetData.StyleParams;
		StyleParams.HorizontalAlignment = HAlign_Fill;
		// Default to vertical fill if vertical alignment hasn't been modified for this particular entry.
		if (!StyleParams.VerticalAlignment.IsSet())
		{
			StyleParams.VerticalAlignment = VAlign_Fill;
		}

		ToolBarBuilder.AddWidget(Widget.ToSharedRef(), StyleParams, Block.TutorialHighlightName, Block.WidgetData.bSearchable, FNewMenuDelegate(), Visibility, ActualResizeParams);
	}
	else
	{
		UE_LOGF(LogToolMenus, Warning, "Toolbar '%ls', item '%ls', Toolbars do not support: %ls",
			*MenuData->MenuName.ToString(), *Block.Name.ToString(), *UEnum::GetValueAsString(Block.Type));
	}

	ToolBarBuilder.EndStyleOverride();
}

void UToolMenus::PopulateToolBarBuilder(FToolBarBuilder& ToolBarBuilder, UToolMenu* MenuData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UToolMenus::PopulateToolBarBuilder);

	if (GetEditMenusMode() && !MenuData->IsEditing() && EditMenuDelegate.IsBound())
	{
		TWeakObjectPtr<UToolMenu> WeakMenuPtr = MenuData;
		const FName MenuName = MenuData->GetMenuName();
		ToolBarBuilder.BeginSection(MenuName);
		ToolBarBuilder.AddToolBarButton(
			FExecuteAction::CreateLambda([MenuName, WeakMenuPtr]()
			{
				FPlatformApplicationMisc::ClipboardCopy(*MenuName.ToString());
				if (UToolMenu* InMenu = WeakMenuPtr.Get())
				{
					UToolMenus::Get()->EditMenuDelegate.ExecuteIfBound(InMenu);
				}
			}), 
			"MenuName",
			LOCTEXT("EditMenu", "Edit Menu"),
			LOCTEXT("EditMenu_ToolTip", "Open menu editor"),
			EditToolbarIcon
		);
		ToolBarBuilder.EndSection();
	}

	// Add the sections grouped by alignment with SSpacers in between. This visually separates them and allows users
	// to align sections to appear first, middle, or last. Default-aligned sections appear grouped with first-aligned
	// sections but appear after them.
	TArray<FToolMenuSection*> SortedSections;
	SortedSections.Reserve(MenuData->Sections.Num());
	for (int32 Index = 0; Index < MenuData->Sections.Num(); ++Index)
	{
		SortedSections.Add(&MenuData->Sections[Index]);
	}
	
	SortedSections.StableSort([](const FToolMenuSection& A, const FToolMenuSection& B)
	{
		return UE::ToolMenus::Private::SortSectionAlignment(A.Alignment, B.Alignment) < 0;
	});
	
	EToolMenuSectionAlign LastAlignment = EToolMenuSectionAlign::First;
	for (int32 SectionIndex = 0; SectionIndex < SortedSections.Num(); ++SectionIndex)
	{
		FToolMenuSection& Section = *SortedSections[SectionIndex];
		
		if (Section.Alignment != LastAlignment)
		{
			const bool bIsMiddleOrLast = Section.Alignment == EToolMenuSectionAlign::Middle || Section.Alignment == EToolMenuSectionAlign::Last;

			// Add a spacer before the middle and last alignment groups, and only if we've already added a section
			if (bIsMiddleOrLast && SectionIndex > 0)
			{
				static constexpr float AlmostZero = UE_KINDA_SMALL_NUMBER; // Using 0.0f results in different behavior, so we just use a small, sub-1 pixel value (which is also used as a proportion of the overall layout space).

				FMenuEntryStyleParams StyleParams;

				StyleParams.HorizontalAlignment = HAlign_Right;
				StyleParams.SizeRule = FSizeParam::ESizeRule::SizeRule_Stretch;
				StyleParams.FillSize = AlmostZero;
				StyleParams.FillSizeMin = AlmostZero; // Allow the spacer to shrink to nothing if another widget needs the space.
				StyleParams.MinimumSize = AlmostZero;
				StyleParams.DesiredWidthOverride = StyleParams.DesiredHeightOverride = AlmostZero;

				FMenuEntryResizeParams ResizeParams;
				// Never show spacers in overflow menus.
				ResizeParams.VisibleInOverflow = false;
				// Prevent our spacers from overflowing. This will keep them in the toolbar, but they will still shrink so
				// overflow of other entries will not be affected. What will be affected, however, is stretching of the
				// toolbar after entries have been clipped. By keeping the stretchers in the toolbar, we can grow them to
				// fill up any space left over when a widget is clipped.
				ResizeParams.AllowClipping = false;

				ToolBarBuilder.AddWidget(
					SNew(SSpacer), StyleParams, NAME_None, false, FNewMenuDelegate(), TAttribute<EVisibility>(), ResizeParams
				);
			}
		}
		
		if (Section.Construct.NewToolBarDelegateLegacy.IsBound())
		{
			Section.Construct.NewToolBarDelegateLegacy.Execute(ToolBarBuilder, MenuData);
			continue;
		}
		
		const bool bIsLastSection = SectionIndex >= SortedSections.Num() - 1;
		// "Default" sections are placed directly after "first" sections, but on the same side.
		// Therefor, they should not be considered "first".
		const bool bFirstSectionInAlignmentGroup = Section.Alignment != LastAlignment && Section.Alignment != EToolMenuSectionAlign::Default;

		const bool bSectionShouldHaveSeparator = MenuData->bSeparateSections && !bFirstSectionInAlignmentGroup;
		ToolBarBuilder.BeginSection(Section.Name, bSectionShouldHaveSeparator, Section.ResizeParams);

		FName PreviousBlockGroupName = NAME_None;
		for (int BlockIndex = 0; BlockIndex < Section.Blocks.Num(); ++BlockIndex)
		{
			const bool bIsLastBlock = BlockIndex >= Section.Blocks.Num() - 1;
			FToolMenuEntry& Block = Section.Blocks[BlockIndex];

			// This is the top level of the toolbar; nothing is being elevated.
			const bool bRaiseToTopLevel = false;
			const bool bIsLastBlockOfLastSection = bIsLastBlock && bIsLastSection;
				
			const FName EntryBlockGroupName = Block.ToolBarData.BlockGroupName;
			if (EntryBlockGroupName != PreviousBlockGroupName)
			{
				if (!PreviousBlockGroupName.IsNone())
				{
					ToolBarBuilder.EndBlockGroup();
				}
				if (!EntryBlockGroupName.IsNone())
				{
					ToolBarBuilder.BeginBlockGroup();
				}
			}
			PreviousBlockGroupName = EntryBlockGroupName;
				
			PopulateToolBarBuilderWithEntry(ToolBarBuilder, MenuData, Section, Block, bRaiseToTopLevel, bIsLastBlockOfLastSection);
		}
			
		if (!PreviousBlockGroupName.IsNone())
		{
			ToolBarBuilder.EndBlockGroup();
		}

		ToolBarBuilder.EndSection();
		
		LastAlignment = Section.Alignment;
	}

	AddReferencedContextObjects(ToolBarBuilder.GetMultiBox(), MenuData);
}

void UToolMenus::PopulateMenuBarBuilder(FMenuBarBuilder& MenuBarBuilder, UToolMenu* MenuData)
{
	for (int i=0; i < MenuData->Sections.Num(); ++i)
	{
		const FToolMenuSection& Section = MenuData->Sections[i];
		for (const FToolMenuEntry& Block : Section.Blocks)
		{
			if (Block.SubMenuData.ConstructMenu.OnGetContent.IsBound())
			{
				MenuBarBuilder.AddPullDownMenu(
					Block.Label,
					Block.ToolTip,
					Block.SubMenuData.ConstructMenu.OnGetContent,
					Block.Name
				);
			}
			else if (Block.SubMenuData.ConstructMenu.NewMenuLegacy.IsBound())
			{
				MenuBarBuilder.AddPullDownMenu(
					Block.Label,
					Block.ToolTip,
					Block.SubMenuData.ConstructMenu.NewMenuLegacy,
					Block.Name
				);
			}
			else
			{
				MenuBarBuilder.AddPullDownMenu(
					Block.Label,
					Block.ToolTip,
					FNewMenuDelegate::CreateUObject(this, &UToolMenus::PopulateSubMenu, TWeakObjectPtr<UToolMenu>(MenuData), Block, Block.Name),
					Block.Name
				);
			}
		}
	}

	const bool bIsEditing = MenuData->IsEditing();
	if (GetEditMenusMode() && !bIsEditing && EditMenuDelegate.IsBound())
	{
		TWeakObjectPtr<UToolMenu> WeakMenuPtr = MenuData;
		const FName MenuName = MenuData->GetMenuName();
		MenuBarBuilder.AddMenuEntry(
			LOCTEXT("EditMenuBar_Label", "Edit Menu"),
			FText::Format(LOCTEXT("EditMenuBar_ToolTip", "Edit Menu: {0}"), FText::FromName(MenuName)),
			EditMenuIcon,
			FExecuteAction::CreateLambda([MenuName, WeakMenuPtr]()
			{
				FPlatformApplicationMisc::ClipboardCopy(*MenuName.ToString());
				if (UToolMenu* InMenu = WeakMenuPtr.Get())
				{
					UToolMenus::Get()->EditMenuDelegate.ExecuteIfBound(InMenu);
				}
			}),
			"MenuName"
		);
	}

	AddReferencedContextObjects(MenuBarBuilder.GetMultiBox(), MenuData);
}

FOnGetContent UToolMenus::ConvertWidgetChoice(const FNewToolMenuChoice& Choice, const FToolMenuContext& Context, const FToolMenuEntry& Block) const
{
	if (Choice.NewToolMenuWidget.IsBound())
	{
		return FOnGetContent::CreateLambda([ToCall = Choice.NewToolMenuWidget, Context]()
		{
			if (ToCall.IsBound())
			{
				return ToCall.Execute(Context);
			}

			return SNullWidget::NullWidget;
		});
	}
	else if (Choice.NewToolMenu.IsBound())
	{
		return FOnGetContent::CreateLambda([ToCall = Choice.NewToolMenu, Context]()
		{
			if (ToCall.IsBound())
			{
				UToolMenu* MenuData = UToolMenus::Get()->NewToolMenuObject(FName(TEXT("NewToolMenu")), NAME_None);
				MenuData->Context = Context;
				ToCall.Execute(MenuData);
				return UToolMenus::Get()->GenerateWidget(MenuData);
			}

			return SNullWidget::NullWidget;
		});
	}
	else if (Choice.NewMenuLegacy.IsBound())
	{
		return FOnGetContent::CreateLambda([ToCall = Choice.NewMenuLegacy, Context, bShouldCloseWindowAfterMenuSelection = Block.bShouldCloseWindowAfterMenuSelection]()
		{
			if (ToCall.IsBound())
			{
				FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Context.CommandList, Context.GetAllExtenders());
				ToCall.Execute(MenuBuilder);
				return MenuBuilder.MakeWidget();
			}

			return SNullWidget::NullWidget;
		});
	}
	return Choice.OnGetContent;
}

FUIAction UToolMenus::ConvertUIAction(const FToolMenuEntry& Block, const FToolMenuContext& Context)
{
	FUIAction UIAction;
	
	if (Block.ScriptObject)
	{
		UIAction = ConvertScriptObjectToUIAction(Block.ScriptObject, Context);
	}
	else
	{
		UIAction = ConvertUIAction(Block.Action, Context);
	}

	if (!UIAction.ExecuteAction.IsBound() && Block.StringExecuteAction.IsBound())
	{
		UIAction.ExecuteAction = Block.StringExecuteAction.ToExecuteAction(Block.Name, Context);
	}

	return UIAction;
}

FUIAction UToolMenus::ConvertUIAction(const FToolUIActionChoice& Choice, const FToolMenuContext& Context)
{
	if (const FToolUIAction* ToolAction = Choice.GetToolUIAction())
	{
		return ConvertUIAction(*ToolAction, Context);
	}
	else if (const FToolDynamicUIAction* DynamicToolAction = Choice.GetToolDynamicUIAction())
	{
		return ConvertUIAction(*DynamicToolAction, Context);
	}
	else if (const FUIAction* Action = Choice.GetUIAction())
	{
		return *Action;
	}

	return FUIAction();
}

FUIAction UToolMenus::ConvertUIAction(const FToolUIAction& Actions, const FToolMenuContext& Context)
{
	FUIAction UIAction;

	if (Actions.ExecuteAction.IsBound())
	{
		UIAction.ExecuteAction.BindLambda([DelegateToCall = Actions.ExecuteAction, Context]()
		{
			DelegateToCall.ExecuteIfBound(Context);
		});
	}

	if (Actions.CanExecuteAction.IsBound())
	{
		UIAction.CanExecuteAction.BindLambda([DelegateToCall = Actions.CanExecuteAction, Context]()
		{
			return DelegateToCall.Execute(Context);
		});
	}

	if (Actions.GetActionCheckState.IsBound())
	{
		UIAction.GetActionCheckState.BindLambda([DelegateToCall = Actions.GetActionCheckState, Context]()
		{
			return DelegateToCall.Execute(Context);
		});
	}

	if (Actions.IsActionVisibleDelegate.IsBound())
	{
		UIAction.IsActionVisibleDelegate.BindLambda([DelegateToCall = Actions.IsActionVisibleDelegate, Context]()
		{
			return DelegateToCall.Execute(Context);
		});
	}

	return UIAction;
}

bool UToolMenus::CanSafelyRouteCall()
{
	return !(GIntraFrameDebuggingGameThread || FUObjectThreadContext::Get().IsRoutingPostLoad);
}

FUIAction UToolMenus::ConvertUIAction(const FToolDynamicUIAction& Actions, const FToolMenuContext& Context)
{
	FUIAction UIAction;

	if (Actions.ExecuteAction.IsBound())
	{
		UIAction.ExecuteAction.BindLambda([DelegateToCall = Actions.ExecuteAction, Context]()
		{
			DelegateToCall.ExecuteIfBound(Context);
		});
	}

	if (Actions.CanExecuteAction.IsBound())
	{
		UIAction.CanExecuteAction.BindLambda([DelegateToCall = Actions.CanExecuteAction, Context]()
		{
			if (DelegateToCall.IsBound() && UToolMenus::CanSafelyRouteCall())
			{
				return DelegateToCall.Execute(Context);
			}

			return false;
		});
	}

	if (Actions.GetActionCheckState.IsBound())
	{
		UIAction.GetActionCheckState.BindLambda([DelegateToCall = Actions.GetActionCheckState, Context]()
		{
			if (DelegateToCall.IsBound() && UToolMenus::CanSafelyRouteCall())
			{
				return DelegateToCall.Execute(Context);
			}

			return ECheckBoxState::Unchecked;
		});
	}

	if (Actions.IsActionVisibleDelegate.IsBound())
	{
		UIAction.IsActionVisibleDelegate.BindLambda([DelegateToCall = Actions.IsActionVisibleDelegate, Context]()
		{
			if (DelegateToCall.IsBound() && UToolMenus::CanSafelyRouteCall())
			{
				return DelegateToCall.Execute(Context);
			}

			return true;
		});
	}

	return UIAction;
}

FUIAction UToolMenus::ConvertScriptObjectToUIAction(UToolMenuEntryScript* ScriptObject, const FToolMenuContext& Context)
{
	FUIAction UIAction;

	if (ScriptObject)
	{
		TWeakObjectPtr<UToolMenuEntryScript> WeakScriptObject(ScriptObject);
		UClass* ScriptClass = ScriptObject->GetClass();

		static const FName ExecuteName = GET_FUNCTION_NAME_CHECKED(UToolMenuEntryScript, Execute);
		if (ScriptClass->IsFunctionImplementedInScript(ExecuteName))
		{
			UIAction.ExecuteAction.BindUFunction(ScriptObject, ExecuteName, Context);
		}

		static const FName CanExecuteName = GET_FUNCTION_NAME_CHECKED(UToolMenuEntryScript, CanExecute);
		if (ScriptClass->IsFunctionImplementedInScript(CanExecuteName))
		{
			UIAction.CanExecuteAction.BindLambda([WeakScriptObject, Context]()
			{
				UToolMenuEntryScript* Object = UToolMenuEntryScript::GetIfCanSafelyRouteCall(WeakScriptObject);
				return Object ? Object->CanExecute(Context) : false;
			});
		}

		static const FName GetCheckStateName = GET_FUNCTION_NAME_CHECKED(UToolMenuEntryScript, GetCheckState);
		if (ScriptClass->IsFunctionImplementedInScript(GetCheckStateName))
		{
			UIAction.GetActionCheckState.BindLambda([WeakScriptObject, Context]()
			{
				UToolMenuEntryScript* Object = UToolMenuEntryScript::GetIfCanSafelyRouteCall(WeakScriptObject);
				return Object ? Object->GetCheckState(Context) : ECheckBoxState::Unchecked;
			});
		}

		static const FName IsVisibleName = GET_FUNCTION_NAME_CHECKED(UToolMenuEntryScript, IsVisible);
		if (ScriptClass->IsFunctionImplementedInScript(IsVisibleName))
		{
			UIAction.IsActionVisibleDelegate.BindLambda([WeakScriptObject, Context]()
			{
				UToolMenuEntryScript* Object = UToolMenuEntryScript::GetIfCanSafelyRouteCall(WeakScriptObject);
				return Object ? Object->IsVisible(Context) : true;
			});
		}
	}

	return UIAction;
}

void UToolMenus::ExecuteStringCommand(const FToolMenuStringCommand StringCommand, FName MenuName, FToolMenuContext Context)
{
	if (StringCommand.IsBound())
	{
		const FName TypeName = StringCommand.GetTypeName();
		UToolMenus* ToolMenus = UToolMenus::Get();
		if (const FToolMenuExecuteString* Handler = ToolMenus->StringCommandHandlers.Find(TypeName))
		{
			if (Handler->IsBound())
			{
				Handler->Execute(StringCommand.String, Context);
			}
		}
		else
		{
			UE_LOGF(LogToolMenus, Warning, "Unknown string command handler type: '%ls'", *TypeName.ToString());
		}

		ToolMenus->OnStringCommandExecuted.Broadcast(MenuName, TypeName);
	}
}

TAttribute<EVisibility> UToolMenus::CalculateToolbarVisibility(UToolMenu* Menu, FToolMenuSection& Section, FToolMenuEntry& Entry, bool bIsRaisingToTopLevel, bool bEmbedActionOrCommand)
{
	// Single values get returned directly. Allow that to cause zero allocations.
	TArray<FToolMenuVisibilityChoice, TInlineAllocator<1>> Visibilities;

	if (Section.Visibility.IsSet())
	{
		Visibilities.Add(Section.Visibility);
	}

	if (bIsRaisingToTopLevel)
	{
		FToolMenuVisibilityChoice ShowInTopLevel = Entry.ShowInToolbarTopLevel;
		if (Entry.ScriptObject)
		{
			// Allow scripts to override ShowInToolbarTopLevel.
			if (const TAttribute<bool> ScriptShowInToolbarTopLevel = Entry.ScriptObject->CreateShowInToolbarTopLevelAttribute(Menu->Context);
				ScriptShowInToolbarTopLevel.IsSet())
			{
				ShowInTopLevel = UE::ToolMenus::Private::CombineVisibility(
					ScriptShowInToolbarTopLevel,
					ShowInTopLevel
				);
			}
		}
		
		if (ShowInTopLevel.IsSet())
		{
			Visibilities.Add(ShowInTopLevel);
		}
	}
	
	if (Entry.Visibility.IsSet())
	{
		Visibilities.Add(Entry.Visibility);
	}
	
	if (bEmbedActionOrCommand)
	{
		if (Entry.Command.IsValid() && !Entry.IsCommandKeybindOnly())
		{
			TSharedPtr<const FUICommandList> CommandListForAction;
			if (const FUIAction* FoundAction = Entry.GetActionForCommand(Menu->Context, CommandListForAction))
			{
				if (FoundAction->IsActionVisibleDelegate.IsBound())
				{
					Visibilities.Add(FoundAction->IsActionVisibleDelegate);
				}
			}
		}
	
		const FUIAction UIAction = [&Entry, Menu]()
		{
			if (Entry.ToolBarData.ActionOverride.IsSet())
			{
				return UToolMenus::ConvertUIAction(Entry.ToolBarData.ActionOverride.GetValue(), Menu->Context);
			}
			else
			{
				return UToolMenus::ConvertUIAction(Entry, Menu->Context);
			}
		}();
		
		if (UIAction.IsActionVisibleDelegate.IsBound())
		{
			Visibilities.Add(UIAction.IsActionVisibleDelegate);
		}
	}
	
	if (Visibilities.Num() == 0)
	{
		return TAttribute<EVisibility>();
	}
	
	if (Visibilities.Num() == 1)
	{
		return Visibilities[0];
	}
	
	return TAttribute<EVisibility>::CreateLambda([Visibilities]
	{
		for (const FToolMenuVisibilityChoice& VisibilityChoice : Visibilities)
		{
			const EVisibility Visibility = VisibilityChoice.Get();
			if (Visibility != EVisibility::Visible)
			{
				return Visibility;
			}
		}
		
		return EVisibility::Visible;
	});
}

UToolMenu* UToolMenus::FindSubMenuToGenerateWith(const FName InParentName, const FName InChildName)
{
	FName BaseName = InParentName;
	while (BaseName != NAME_None)
	{
		FName JoinedName = JoinMenuPaths(BaseName, InChildName);
		if (UToolMenu* Found = FindMenu(JoinedName))
		{
			return Found;
		}

		UToolMenu* BaseData = FindMenu(BaseName);
		BaseName = BaseData ? BaseData->MenuParent : NAME_None;
	}

	return nullptr;
}

UObject* UToolMenus::FindContext(const FToolMenuContext& InContext, UClass* InClass)
{
	return InContext.FindByClass(InClass);
}

void UToolMenus::AddReferencedContextObjects(const TSharedRef<FMultiBox>& InMultiBox, const UToolMenu* InMenu)
{
	if (InMenu)
	{
		auto& References = WidgetObjectReferences.FindOrAdd(InMultiBox);
		References.AddUnique(InMenu);
		for (const TWeakObjectPtr<UObject> WeakObject : InMenu->Context.ContextObjects)
		{
			if (UObject* Object = WeakObject.Get())
			{
				References.AddUnique(Object);
			}
		}
	}
}

void UToolMenus::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UToolMenus* This = CastChecked<UToolMenus>(InThis);

	for (auto It = This->WidgetObjectReferences.CreateIterator(); It; ++It)
	{
		if (It->Key.IsValid())
		{
			Collector.AddReferencedObjects(It->Value, InThis);
		}
		else
		{
			It.RemoveCurrent();
		}
	}

	for (auto WidgetsForMenuNameIt = This->GeneratedMenuWidgets.CreateIterator(); WidgetsForMenuNameIt; ++WidgetsForMenuNameIt)
	{
		TSharedPtr<FGeneratedToolMenuWidgets>& WidgetsForMenuName = WidgetsForMenuNameIt->Value;

		for (auto Instance = WidgetsForMenuName->Instances.CreateIterator(); Instance; ++Instance)
		{
			if (Instance->Get()->Widget.IsValid())
			{
				Collector.AddReferencedObject(Instance->Get()->GeneratedMenu, InThis);
			}
			else
			{
				Instance.RemoveCurrent();
			}
		}

		if (WidgetsForMenuName->Instances.Num() == 0)
		{
			WidgetsForMenuNameIt.RemoveCurrent();
		}
	}

	Super::AddReferencedObjects(InThis, Collector);
}

UToolMenu* UToolMenus::GenerateMenuOrSubMenuForEdit(const UToolMenu* InMenu)
{
	// Make copy of context so we can set bIsEditing flag on it
	FToolMenuContext NewMenuContext = InMenu->Context;
	NewMenuContext.bIsEditing = true;

	if (!InMenu->SubMenuParent)
	{
		return GenerateMenu(InMenu->GetMenuName(), NewMenuContext);
	}

	// Generate each menu leading up to the final submenu because sub-menus are not required to be registered
	TArray<const UToolMenu*> SubMenuChain = InMenu->GetSubMenuChain();
	if (SubMenuChain.Num() > 0)
	{
		UToolMenu* CurrentGeneratedMenu = GenerateMenu(SubMenuChain[0]->GetMenuName(), NewMenuContext);
		for (int32 i=1; i < SubMenuChain.Num(); ++i)
		{
			if (UToolMenu* Menu = GenerateSubMenu(CurrentGeneratedMenu, SubMenuChain[i]->SubMenuSourceEntryName))
			{
				CurrentGeneratedMenu = Menu;
			}
			else
			{
				return nullptr;
			}
		}

		return CurrentGeneratedMenu;
	}

	return nullptr;
}

void UToolMenus::AddMenuSubstitutionDuringGenerate(const FName OriginalMenu, const FName NewMenu)
{
	MenuSubstitutionsDuringGenerate.Add(OriginalMenu, NewMenu);
}

void UToolMenus::RemoveSubstitutionDuringGenerate(const FName InMenu)
{
	if (const FName* FoundOverrideMenuName = MenuSubstitutionsDuringGenerate.Find(InMenu))
	{
		const FName OverrideMenuName = *FoundOverrideMenuName;

		// Update all active widget instances of this menu
		if (TSharedPtr<FGeneratedToolMenuWidgets> OverrideMenuWidgets = GeneratedMenuWidgets.FindRef(OverrideMenuName))
		{
			if (TSharedPtr<FGeneratedToolMenuWidgets> DestMenuWidgets = GeneratedMenuWidgets.FindRef(InMenu))
			{
				DestMenuWidgets->Instances.Append(OverrideMenuWidgets->Instances);
			}
			else
			{
				GeneratedMenuWidgets.Add(InMenu, OverrideMenuWidgets);
			}

			GeneratedMenuWidgets.Remove(OverrideMenuName);
		}

		MenuSubstitutionsDuringGenerate.Remove(InMenu);

		CleanupStaleWidgetsNextTick();
	}
}

UToolMenu* UToolMenus::GenerateMenu(const FName Name, const FToolMenuContext& InMenuContext)
{
	return GenerateMenuFromHierarchy(CollectHierarchy(Name), InMenuContext);
}

UToolMenu* UToolMenus::GenerateMenuFromHierarchy(const TArray<UToolMenu*>& Hierarchy, const FToolMenuContext& InMenuContext)
{
	UToolMenu* GeneratedMenu = NewToolMenuObject(FName(TEXT("GeneratedMenuFromHierarchy")), NAME_None);

	if (Hierarchy.Num() > 0)
	{
		GeneratedMenu->InitGeneratedCopy(Hierarchy[0], Hierarchy.Last()->MenuName, &InMenuContext);
		for (UToolMenu* HiearchyItem : Hierarchy)
		{
			if (HiearchyItem && !HiearchyItem->bExtendersEnabled)
			{
				GeneratedMenu->SetExtendersEnabled(false);
				break;
			}
		}
		AssembleMenuHierarchy(GeneratedMenu, Hierarchy);
	}

	return GeneratedMenu;
}

TSharedRef<SWidget> UToolMenus::GenerateWidget(const FName InName, const FToolMenuContext& InMenuContext)
{
	OnPreGenerateWidget.Broadcast(InName, InMenuContext);

	UToolMenu* Generated = GenerateMenu(InName, InMenuContext);
	TSharedRef<SWidget> Result = GenerateWidget(Generated);

	OnPostGenerateWidget.Broadcast(InName, Generated);
	
	return Result;
}

TSharedRef<SWidget> UToolMenus::GenerateWidget(const TArray<UToolMenu*>& Hierarchy, const FToolMenuContext& InMenuContext)
{
	if (Hierarchy.Num() == 0)
	{
		return SNullWidget::NullWidget;
	}

	UToolMenu* Generated = GenerateMenuFromHierarchy(Hierarchy, InMenuContext);
	return GenerateWidget(Generated);
}

TSharedRef<SWidget> UToolMenus::GenerateWidget(UToolMenu* GeneratedMenu)
{
	CleanupStaleWidgetsNextTick();

	const ISlateStyle* StyleSetNotNull = GeneratedMenu->GetStyleSet();
	const bool bHadStyleSet = StyleSetNotNull != nullptr;
	if (!bHadStyleSet)
	{
		// Avoid crash when style sets are unregistered/deleted, GetStyleSet() will report an ensure when that happens but return null and menu builders crash when passed null StyleSet.
		StyleSetNotNull = &FCoreStyle::Get();
	}

	TSharedPtr<SWidget> GeneratedWidget;
	if (GeneratedMenu->IsEditing())
	{
		// Convert toolbar into menu during editing
		if (GeneratedMenu->MenuType == EMultiBoxType::ToolBar || GeneratedMenu->MenuType == EMultiBoxType::VerticalToolBar || GeneratedMenu->MenuType == EMultiBoxType::UniformToolBar || GeneratedMenu->MenuType == EMultiBoxType::SlimHorizontalToolBar || GeneratedMenu->MenuType == EMultiBoxType::SlimWrappingToolBar || GeneratedMenu->MenuType == EMultiBoxType::SlimHorizontalUniformToolBar)
		{
			for (FToolMenuSection& Section : GeneratedMenu->Sections)
			{
				for (FToolMenuEntry& Entry : Section.Blocks)
				{
					ModifyEntryForEditDialog(Entry);
				}
			}
		}

		FMenuBuilder MenuBuilder(GeneratedMenu->bShouldCloseWindowAfterMenuSelection, GeneratedMenu->Context.CommandList, GeneratedMenu->Context.GetAllExtenders(), GeneratedMenu->bCloseSelfOnly, StyleSetNotNull, GeneratedMenu->bSearchable, GeneratedMenu->MenuName);

		// Default consistent style is applied, necessary for toolbars to be displayed as menus
		//if (bHadStyleSet && GeneratedMenu->StyleName != NAME_None)
		//{
		//	MenuBuilder.SetStyle(StyleSetNotNull, GeneratedMenu->StyleName);
		//}

		MenuBuilder.SetExtendersEnabled(GeneratedMenu->bExtendersEnabled);
		PopulateMenuBuilder(MenuBuilder, GeneratedMenu);
		if (GeneratedMenu->ModifyBlockWidgetAfterMake.IsBound())
		{
			MenuBuilder.GetMultiBox()->ModifyBlockWidgetAfterMake = GeneratedMenu->ModifyBlockWidgetAfterMake;
		}
		TSharedRef<SWidget> Result = MenuBuilder.MakeWidget();
		GeneratedWidget = Result;
	}
	else if (GeneratedMenu->MenuType == EMultiBoxType::Menu)
	{
		FMenuBuilder MenuBuilder(GeneratedMenu->bShouldCloseWindowAfterMenuSelection, GeneratedMenu->Context.CommandList, GeneratedMenu->Context.GetAllExtenders(), GeneratedMenu->bCloseSelfOnly, StyleSetNotNull, GeneratedMenu->bSearchable, GeneratedMenu->MenuName);

		if (bHadStyleSet && GeneratedMenu->StyleName != NAME_None)
		{
			MenuBuilder.SetStyle(StyleSetNotNull, GeneratedMenu->StyleName);
		}

		MenuBuilder.SetExtendersEnabled(GeneratedMenu->bExtendersEnabled);
		PopulateMenuBuilder(MenuBuilder, GeneratedMenu);
		TSharedRef<SWidget> Result = MenuBuilder.MakeWidget(nullptr, GeneratedMenu->MaxHeight);
		GeneratedWidget = Result;
	}
	else if (GeneratedMenu->MenuType == EMultiBoxType::MenuBar)
	{
		FMenuBarBuilder MenuBarBuilder(GeneratedMenu->Context.CommandList, GeneratedMenu->Context.GetAllExtenders(), StyleSetNotNull, GeneratedMenu->MenuName);

		if (bHadStyleSet && GeneratedMenu->StyleName != NAME_None)
		{
			MenuBarBuilder.SetStyle(StyleSetNotNull, GeneratedMenu->StyleName);
		}

		MenuBarBuilder.SetExtendersEnabled(GeneratedMenu->bExtendersEnabled);
		PopulateMenuBarBuilder(MenuBarBuilder, GeneratedMenu);
		TSharedRef<SWidget> Result = MenuBarBuilder.MakeWidget();
		GeneratedWidget = Result;
	}
	else if (GeneratedMenu->MenuType == EMultiBoxType::ToolBar || GeneratedMenu->MenuType == EMultiBoxType::VerticalToolBar || GeneratedMenu->MenuType == EMultiBoxType::UniformToolBar || GeneratedMenu->MenuType == EMultiBoxType::SlimHorizontalToolBar || GeneratedMenu->MenuType == EMultiBoxType::SlimWrappingToolBar  || GeneratedMenu->MenuType == EMultiBoxType::SlimHorizontalUniformToolBar)
	{
		FToolBarBuilder ToolbarBuilder(GeneratedMenu->MenuType, GeneratedMenu->Context.CommandList, GeneratedMenu->MenuName, GeneratedMenu->Context.GetAllExtenders(), GeneratedMenu->bToolBarForceSmallIcons);
		ToolbarBuilder.SetExtendersEnabled(GeneratedMenu->bExtendersEnabled);
		ToolbarBuilder.SetIsFocusable(GeneratedMenu->bToolBarIsFocusable);
		ToolbarBuilder.SetAllowWrapButton(GeneratedMenu->bAllowToolBarWrapButton);

		if (bHadStyleSet && GeneratedMenu->StyleName != NAME_None)
		{
			ToolbarBuilder.SetStyle(StyleSetNotNull, GeneratedMenu->StyleName);
		}

		PopulateToolBarBuilder(ToolbarBuilder, GeneratedMenu);
		TSharedRef<SWidget> Result = ToolbarBuilder.MakeWidget();
		GeneratedWidget = Result;
	}

	TSharedPtr<FGeneratedToolMenuWidgets>& WidgetsForMenuName = GeneratedMenuWidgets.FindOrAdd(GeneratedMenu->MenuName);
	if (!WidgetsForMenuName.IsValid())
	{
		WidgetsForMenuName = MakeShared<FGeneratedToolMenuWidgets>();
	}

	// Store a copy so that we can call 'Refresh' on menus not in the database
	FGeneratedToolMenuWidget& GeneratedMenuWidget = *WidgetsForMenuName->Instances.Add_GetRef(MakeShared<FGeneratedToolMenuWidget>());
	GeneratedMenuWidget.OriginalMenu = GeneratedMenu;
	GeneratedMenuWidget.GeneratedMenu = DuplicateObject<UToolMenu>(GeneratedMenu, this, MakeUniqueObjectName(this, UToolMenus::StaticClass(), FName("MenuForRefresh")));
	GeneratedMenuWidget.GeneratedMenu->bShouldCleanupContextOnDestroy = true;
	// Copy native properties that serialize does not
	GeneratedMenuWidget.GeneratedMenu->Context = GeneratedMenu->Context;
	GeneratedMenuWidget.GeneratedMenu->StyleSetName = GeneratedMenu->StyleSetName;
	GeneratedMenuWidget.GeneratedMenu->StyleName = GeneratedMenu->StyleName;

	if (GeneratedWidget)
	{
		GeneratedMenuWidget.Widget = GeneratedWidget;
		return GeneratedWidget.ToSharedRef();
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

void UToolMenus::ModifyEntryForEditDialog(FToolMenuEntry& Entry)
{
	if (Entry.Type == EMultiBlockType::ToolBarButton)
	{
		Entry.Type = EMultiBlockType::MenuEntry;
	}
	else if (Entry.Type == EMultiBlockType::ToolBarComboButton)
	{
		Entry.Type = EMultiBlockType::MenuEntry;
		if (Entry.ToolBarData.bSimpleComboBox)
		{
			Entry.SubMenuData.bIsSubMenu = true;
		}
	}
}

void UToolMenus::AssignSetTimerForNextTickDelegate(const FSimpleDelegate& InDelegate)
{
	SetTimerForNextTickDelegate = InDelegate;
}

void UToolMenus::SetNextTickTimer()
{
	if (!bNextTickTimerIsSet)
	{
		if (SetTimerForNextTickDelegate.IsBound())
		{
			bNextTickTimerIsSet = true;
			SetTimerForNextTickDelegate.Execute();
		}
	}
}

void UToolMenus::CleanupStaleWidgetsNextTick(bool bGarbageCollect)
{
	bCleanupStaleWidgetsNextTick = true;

	if (bGarbageCollect)
	{
		bCleanupStaleWidgetsNextTickGC = true;
	}

	SetNextTickTimer();
}

void UToolMenus::RefreshAllWidgets()
{
	if (!bSuppressRefreshWidgetsRequests)
	{
		bRefreshWidgetsNextTick = true;
		SetNextTickTimer();
	}
}

void UToolMenus::HandleNextTick()
{
	if (bCleanupStaleWidgetsNextTick || bRefreshWidgetsNextTick)
	{
		CleanupStaleWidgets();
		bCleanupStaleWidgetsNextTick = false;
		bCleanupStaleWidgetsNextTickGC = false;

		if (bRefreshWidgetsNextTick)
		{
			TGuardValue<bool> SuppressRefreshWidgetsRequestsGuard(bSuppressRefreshWidgetsRequests, true);

			// Copy before enumerate because modified inside RefreshMenuWidget
			TMap<FName, TSharedPtr<FGeneratedToolMenuWidgets>> GeneratedMenuWidgetsCopy = GeneratedMenuWidgets;
			for (TPair<FName, TSharedPtr<FGeneratedToolMenuWidgets>>& WidgetsForMenuNameIt : GeneratedMenuWidgetsCopy)
			{
				// Copy before enumerate because modified inside RefreshMenuWidget
				TArray<TSharedPtr<FGeneratedToolMenuWidget>> InstancesCopy = WidgetsForMenuNameIt.Value->Instances;
				for (TSharedPtr<FGeneratedToolMenuWidget>& Instance : InstancesCopy)
				{
					if (Instance->Widget.IsValid())
					{
						RefreshMenuWidget(WidgetsForMenuNameIt.Key, *Instance);
					}
				}
			}

			bRefreshWidgetsNextTick = false;
		}
	}

	bNextTickTimerIsSet = false;
}

void UToolMenus::CleanupStaleWidgets()
{
	bool bModified = false;
	for (auto WidgetsForMenuNameIt = GeneratedMenuWidgets.CreateIterator(); WidgetsForMenuNameIt; ++WidgetsForMenuNameIt)
	{
		TSharedPtr<FGeneratedToolMenuWidgets>& WidgetsForMenuName = WidgetsForMenuNameIt->Value;

		for (auto Instance = WidgetsForMenuName->Instances.CreateIterator(); Instance; ++Instance)
		{
			if (!(*Instance)->Widget.IsValid())
			{
				bModified = true;
				Instance.RemoveCurrent();
			}
		}

		if (WidgetsForMenuName->Instances.Num() == 0)
		{
			bModified = true;
			WidgetsForMenuNameIt.RemoveCurrent();
		}
	}

	if (bModified && bCleanupStaleWidgetsNextTickGC && !IsAsyncLoading())
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
}

bool UToolMenus::RefreshMenuWidget(const FName InName)
{
	bool bRefreshedAnyWidget = false;

	// Copy the TSharedPtr because RefreshMenuWidget can modify GeneratedMenuWidgets and delete any memory we point to
	if (TSharedPtr<FGeneratedToolMenuWidgets> WidgetsForMenuName = GeneratedMenuWidgets.FindRef(InName); WidgetsForMenuName.IsValid())
	{
		// Copy before enumerate because entries can be added to this array inside RefreshMenuWidget
		TArray<TSharedPtr<FGeneratedToolMenuWidget>> InstancesCopy = WidgetsForMenuName->Instances;
		for (TSharedPtr<FGeneratedToolMenuWidget>& Instance : InstancesCopy)
		{
			if (RefreshMenuWidget(InName, *Instance))
			{
				bRefreshedAnyWidget = true;
			}
			else
			{
				// Remove from original instead of InstancesCopy
				WidgetsForMenuName->Instances.Remove(Instance);
			}
		}
	}

	return bRefreshedAnyWidget;
}

bool UToolMenus::RefreshMenuWidget(const FName InName, FGeneratedToolMenuWidget& GeneratedMenuWidget)
{
	if (!GeneratedMenuWidget.Widget.IsValid())
	{
		return false;
	}

	// Regenerate menu from database
	GeneratedMenuWidget.GeneratedMenu->bShouldCleanupContextOnDestroy = false; // The new menu will do this

	// GeneratedMenuWidget.GeneratedMenu is a copy of the original menu, so we also need to make sure the original menu does not clean up its context
	if(UToolMenu* OriginalMenu = GeneratedMenuWidget.OriginalMenu.Get())
	{
		OriginalMenu->bShouldCleanupContextOnDestroy = false;
	}

	UToolMenu* GeneratedMenu = GenerateMenu(InName, GeneratedMenuWidget.GeneratedMenu->Context);
	GeneratedMenuWidget.GeneratedMenu = GeneratedMenu;

	const ISlateStyle* StyleSetNotNull = GeneratedMenu->GetStyleSet();
	const bool bHadStyleSet = StyleSetNotNull != nullptr;
	if (!bHadStyleSet)
	{
		// Avoid crash when style sets are unregistered/deleted, GetStyleSet() will report an ensure when that happens but return null and menu builders crash when passed null StyleSet.
		StyleSetNotNull = &FCoreStyle::Get();
	}

	// Regenerate Multibox
	TSharedRef<SMultiBoxWidget> MultiBoxWidget = StaticCastSharedRef<SMultiBoxWidget>(GeneratedMenuWidget.Widget.Pin().ToSharedRef());
	if (GeneratedMenu->MenuType == EMultiBoxType::Menu)
	{
		FMenuBuilder MenuBuilder(GeneratedMenu->bShouldCloseWindowAfterMenuSelection, GeneratedMenu->Context.CommandList, GeneratedMenu->Context.GetAllExtenders(), GeneratedMenu->bCloseSelfOnly, StyleSetNotNull, GeneratedMenu->bSearchable);
		MenuBuilder.SetExtendersEnabled(GeneratedMenu->bExtendersEnabled);

		if (bHadStyleSet && GeneratedMenu->StyleName != NAME_None)
		{
			MenuBuilder.SetStyle(StyleSetNotNull, GeneratedMenu->StyleName);
		}

		PopulateMenuBuilder(MenuBuilder, GeneratedMenu);
		MultiBoxWidget->SetMultiBox(MenuBuilder.GetMultiBox());
	}
	else if (GeneratedMenu->MenuType == EMultiBoxType::MenuBar)
	{
		FMenuBarBuilder MenuBarBuilder(GeneratedMenu->Context.CommandList, GeneratedMenu->Context.GetAllExtenders(), StyleSetNotNull);
		MenuBarBuilder.SetExtendersEnabled(GeneratedMenu->bExtendersEnabled);

		if (bHadStyleSet && GeneratedMenu->StyleName != NAME_None)
		{
			MenuBarBuilder.SetStyle(StyleSetNotNull, GeneratedMenu->StyleName);
		}

		PopulateMenuBarBuilder(MenuBarBuilder, GeneratedMenu);
		MultiBoxWidget->SetMultiBox(MenuBarBuilder.GetMultiBox());
	}
	else if (GeneratedMenu->MenuType == EMultiBoxType::ToolBar || GeneratedMenu->MenuType == EMultiBoxType::VerticalToolBar || GeneratedMenu->MenuType == EMultiBoxType::UniformToolBar || GeneratedMenu->MenuType == EMultiBoxType::SlimHorizontalToolBar || GeneratedMenu->MenuType == EMultiBoxType::SlimWrappingToolBar || GeneratedMenu->MenuType == EMultiBoxType::SlimHorizontalUniformToolBar)
	{
		FToolBarBuilder ToolbarBuilder(GeneratedMenu->MenuType, GeneratedMenu->Context.CommandList, GeneratedMenu->MenuName, GeneratedMenu->Context.GetAllExtenders(), GeneratedMenu->bToolBarForceSmallIcons);
		ToolbarBuilder.SetExtendersEnabled(GeneratedMenu->bExtendersEnabled);
		ToolbarBuilder.SetIsFocusable(GeneratedMenu->bToolBarIsFocusable);
		ToolbarBuilder.SetAllowWrapButton(GeneratedMenu->bAllowToolBarWrapButton);

		if (bHadStyleSet && GeneratedMenu->StyleName != NAME_None)
		{
			ToolbarBuilder.SetStyle(StyleSetNotNull, GeneratedMenu->StyleName);
		}

		PopulateToolBarBuilder(ToolbarBuilder, GeneratedMenu);
		MultiBoxWidget->SetMultiBox(ToolbarBuilder.GetMultiBox());
	}

	MultiBoxWidget->BuildMultiBoxWidget();
	return true;
}

UToolMenu* UToolMenus::GenerateMenuAsBuilder(const UToolMenu* InMenu, const FToolMenuContext& InMenuContext)
{
	TArray<UToolMenu*> Hierarchy = CollectHierarchy(InMenu->MenuName);

	// Insert InMenu as second to last so items in InMenu appear before items registered in database by other plugins
	if (Hierarchy.Num() > 0)
	{
		Hierarchy.Insert((UToolMenu*)InMenu, Hierarchy.Num() - 1);
	}
	else
	{
		Hierarchy.Add((UToolMenu*)InMenu);
	}

	return GenerateMenuFromHierarchy(Hierarchy, InMenuContext);
}

UToolMenu* UToolMenus::RegisterMenu(const FName InName, const FName InParent, EMultiBoxType InType, bool bWarnIfAlreadyRegistered)
{
	if (UToolMenu* Found = FindMenu(InName))
	{
		if (!Found->bRegistered)
		{
			Found->MenuParent = InParent;
			Found->MenuType = InType;
			Found->MenuOwner = CurrentOwner();
			Found->bRegistered = true;
			Found->bIsRegistering = true;
			for (FToolMenuSection& Section : Found->Sections)
			{
				Section.bIsRegistering = Found->bIsRegistering;
			}
		}
		else if (bWarnIfAlreadyRegistered)
		{
			UE_LOGF(LogToolMenus, Warning, "Menu already registered : %ls", *InName.ToString());
		}

		return Found;
	}

	UToolMenu* ToolMenu = NewToolMenuObject(FName(TEXT("RegisteredMenu")), InName);
	ToolMenu->InitMenu(CurrentOwner(), InName, InParent, InType);
	ToolMenu->bRegistered = true;
	ToolMenu->bIsRegistering = true;
	Menus.Add(InName, ToolMenu);
	return ToolMenu;
}

UToolMenu* UToolMenus::ExtendMenu(const FName InName)
{
	if (UToolMenu* Found = FindMenu(InName))
	{
		Found->bIsRegistering = false;
		for (FToolMenuSection& Section : Found->Sections)
		{
			Section.bIsRegistering = Found->bIsRegistering;
		}

		// Refresh all widgets because this could be child of another menu being displayed
		RefreshAllWidgets();

		return Found;
	}

	UToolMenu* ToolMenu = NewToolMenuObject(FName(TEXT("RegisteredMenu")), InName);
	ToolMenu->bRegistered = false;
	ToolMenu->bIsRegistering = false;
	Menus.Add(InName, ToolMenu);
	return ToolMenu;
}

UToolMenu* UToolMenus::NewToolMenuObject(const FName NewBaseName, const FName InMenuName)
{
	FName UniqueObjectName = MakeUniqueObjectName(this, UToolMenus::StaticClass(), NewBaseName);
	UToolMenu* Result = NewObject<UToolMenu>(this, UniqueObjectName);
	Result->MenuName = InMenuName;
	return Result;
}

void UToolMenus::RemoveMenu(const FName MenuName)
{
	Menus.Remove(MenuName);
}

bool UToolMenus::AddMenuEntryObject(UToolMenuEntryScript* MenuEntryObject)
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(MenuEntryObject->Data.Menu);
	Menu->AddMenuEntryObject(MenuEntryObject);
	return true;
}

bool UToolMenus::RemoveMenuEntryObject(UToolMenuEntryScript* MenuEntryObject)
{
	if (UToolMenu* Menu = UToolMenus::Get()->FindMenu(MenuEntryObject->Data.Menu))
	{
		Menu->RemoveMenuEntryObject(MenuEntryObject);
		return true;
	}
	return false;
}

void UToolMenus::SetSectionLabel(const FName MenuName, const FName SectionName, const FText Label)
{
	ExtendMenu(MenuName)->FindOrAddSection(SectionName).Label = TAttribute<FText>(Label);
}

void UToolMenus::SetSectionPosition(const FName MenuName, const FName SectionName, const FName PositionName, const EToolMenuInsertType PositionType)
{
	ExtendMenu(MenuName)->FindOrAddSection(SectionName).InsertPosition = FToolMenuInsert(PositionName, PositionType);
}

void UToolMenus::AddSection(const FName MenuName, const FName SectionName, const TAttribute< FText >& InLabel, const FToolMenuInsert InPosition)
{
	UToolMenu* Menu = ExtendMenu(MenuName);
	FToolMenuSection* Section = Menu->FindSection(SectionName);
	if (!Section)
	{
		Menu->AddSection(SectionName, InLabel, InPosition);
	}
}

void UToolMenus::RemoveSection(const FName MenuName, const FName InSection)
{
	if (UToolMenu* Menu = FindMenu(MenuName))
	{
		Menu->RemoveSection(InSection);
	}
}

void UToolMenus::AddEntry(const FName MenuName, const FName InSection, const FToolMenuEntry& InEntry)
{
	ExtendMenu(MenuName)->FindOrAddSection(InSection).AddEntry(InEntry);
}

void UToolMenus::RemoveEntry(const FName MenuName, const FName InSection, const FName InName)
{
	if (UToolMenu* Menu = FindMenu(MenuName))
	{
		if (FToolMenuSection* Section = Menu->FindSection(InSection))
		{
			Section->RemoveEntry(InName);
		}
	}
}

void UToolMenus::UnregisterOwnerInternal(FToolMenuOwner InOwner)
{
	if (InOwner == FToolMenuOwner())
	{
		return;
	}

	bool bNeedsRefresh = false;

	for (const TPair<FName, TObjectPtr<UToolMenu>>& Pair : Menus)
	{
		UToolMenu* Menu = Pair.Value;
		for (int32 SectionIndex = Menu->Sections.Num() - 1; SectionIndex >= 0; --SectionIndex)
		{
			FToolMenuSection& Section = Menu->Sections[SectionIndex];
			if (Section.RemoveEntriesByOwner(InOwner) > 0)
			{
				bNeedsRefresh = true;
			}

			if (Section.Owner == InOwner)
			{
				if (Section.Construct.IsBound())
				{
					Section.Construct = FNewSectionConstructChoice();
					bNeedsRefresh = true;
				}

				if (Section.ToolMenuSectionDynamic)
				{
					Section.ToolMenuSectionDynamic = nullptr;
					bNeedsRefresh = true;
				}

				if (Section.Blocks.Num() == 0)
				{
					Menu->Sections.RemoveAt(SectionIndex, EAllowShrinking::No);
					bNeedsRefresh = true;
				}
			}
		}
	}

	// Refresh any widgets that are currently displayed to the user
	if (bNeedsRefresh)
	{
		RefreshAllWidgets();
	}
}

void UToolMenus::UnregisterRuntimeMenuCustomizationOwner(const FName InOwnerName)
{
	if (InOwnerName.IsNone())
	{
		return;
	}

	bool bNeedsRefresh = false;
	for (FCustomizedToolMenu& CustomizedToolMenu : RuntimeCustomizedMenus)
	{
		if (CustomizedToolMenu.MenuPermissions.UnregisterOwner(InOwnerName))
		{
			bNeedsRefresh = true;
		}

		if (CustomizedToolMenu.SuppressExtenders.Remove(InOwnerName) > 0)
		{
			bNeedsRefresh = true;
		}
	}

	// Refresh any widgets that are currently displayed to the user
	if (bNeedsRefresh)
	{
		RefreshAllWidgets();
	}
}

void UToolMenus::UnregisterRuntimeMenuProfileOwner(const FName InOwnerName)
{
	if (InOwnerName.IsNone())
	{
		return;
	}

	bool bNeedsRefresh = false;

	// Loop through all menus with profiles
	for (TPair<FName, FToolMenuProfileMap>& MenusWithProfiles : RuntimeMenuProfiles)
	{
		// Loop through all profiles for a given menu
		for (TPair<FName, FToolMenuProfile>& MenuProfile : MenusWithProfiles.Value.MenuProfiles)
		{
			if (MenuProfile.Value.MenuPermissions.UnregisterOwner(InOwnerName))
			{
				bNeedsRefresh = true;
			}

			if (MenuProfile.Value.SuppressExtenders.Remove(InOwnerName) > 0)
			{
				bNeedsRefresh = true;
			}
		}
	}

	// Refresh any widgets that are currently displayed to the user
	if (bNeedsRefresh)
	{
		RefreshAllWidgets();
	}
}


FToolMenuOwner UToolMenus::CurrentOwner() const
{
	if (OwnerStack.Num() > 0)
	{
		return OwnerStack.Last();
	}

	return FToolMenuOwner();
}

void UToolMenus::PushOwner(const FToolMenuOwner InOwner)
{
	OwnerStack.Add(InOwner);
}

void UToolMenus::PopOwner(const FToolMenuOwner InOwner)
{
	FToolMenuOwner PoppedOwner = OwnerStack.Pop(EAllowShrinking::No);
	check(PoppedOwner == InOwner);
}

void UToolMenus::UnregisterOwnerByName(FName InOwnerName)
{
	UnregisterOwnerInternal(InOwnerName);
}

void UToolMenus::RegisterStringCommandHandler(const FName InName, const FToolMenuExecuteString& InDelegate)
{
	StringCommandHandlers.Add(InName, InDelegate);
}

void UToolMenus::UnregisterStringCommandHandler(const FName InName)
{
	StringCommandHandlers.Remove(InName);
}

FDelegateHandle UToolMenus::RegisterStartupCallback(const FSimpleMulticastDelegate::FDelegate& InDelegate)
{
	if (IsToolMenuUIEnabled() && UToolMenus::TryGet())
	{
		// Call immediately if systems are initialized
		InDelegate.Execute();
	}
	else
	{
		// Defer call to occur after systems are initialized (slate and menus)
		FDelegateHandle Result = StartupCallbacks.Add(InDelegate);

		if (!InternalStartupCallbackHandle.IsSet())
		{
			InternalStartupCallbackHandle = FCoreDelegates::GetOnPostEngineInit().Add(FSimpleMulticastDelegate::FDelegate::CreateStatic(&UToolMenus::PrivateStartupCallback));
		}

		return Result;
	}

	return FDelegateHandle();
}

void UToolMenus::UnRegisterStartupCallback(FDelegateUserObjectConst UserPointer)
{
	StartupCallbacks.RemoveAll(UserPointer);
}

void UToolMenus::UnRegisterStartupCallback(FDelegateHandle InHandle)
{
	StartupCallbacks.Remove(InHandle);
}

void UToolMenus::PrivateStartupCallback()
{
	UnregisterPrivateStartupCallback();

	if (IsToolMenuUIEnabled() && UToolMenus::TryGet())
	{
		StartupCallbacks.Broadcast();
		StartupCallbacks.Clear();
	}
}

void UToolMenus::UnregisterPrivateStartupCallback()
{
	if (InternalStartupCallbackHandle.IsSet())
	{
		FDelegateHandle& Handle = InternalStartupCallbackHandle.GetValue();
		if (Handle.IsValid())
		{
			FCoreDelegates::GetOnPostEngineInit().Remove(Handle);
			Handle.Reset();
		}
	}
}

void UToolMenus::SaveCustomizations()
{
	SaveConfig();
}

void UToolMenus::RemoveAllCustomizations()
{
	CustomizedMenus.Reset();
}

namespace UE::ToolMenus
{

FToolMenuTestInstanceScoped::FToolMenuTestInstanceScoped()
{
	ScopedInstance = Private::CreateToolMenusInstance();

	PreviousInstance = UToolMenus::Get();

	UToolMenus::Singleton = ScopedInstance;
}

FToolMenuTestInstanceScoped::~FToolMenuTestInstanceScoped()
{
	// Reinstate the previous singleton.
	UToolMenus::Singleton = PreviousInstance;

	// Remove our scoped instance from the root after reinstating the previous instance to not risk our scoped instance
	// from destroying and taking down the whole ToolMenus system with it.
	ScopedInstance->RemoveFromRoot();
}

} // namespace UE::ToolMenus

#undef LOCTEXT_NAMESPACE

