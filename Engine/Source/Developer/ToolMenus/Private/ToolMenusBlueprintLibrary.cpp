// Copyright Epic Games, Inc. All Rights Reserved.


#include "ToolMenusBlueprintLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolMenusBlueprintLibrary)

FScriptSlateIcon UToolMenuEntryExtensions::MakeScriptSlateIcon(const FName StyleSetName, const FName StyleName, const FName SmallStyleName)
{
	if (SmallStyleName == NAME_None)
	{
		return FScriptSlateIcon(StyleSetName, StyleName);
	}
	else
	{
		return FScriptSlateIcon(StyleSetName, StyleName, SmallStyleName);
	}
}

void UToolMenuEntryExtensions::BreakScriptSlateIcon(const FScriptSlateIcon& InValue, FName& StyleSetName, FName& StyleName, FName& SmallStyleName)
{
	StyleSetName = InValue.StyleSetName;
	StyleName = InValue.StyleName;
	SmallStyleName = InValue.SmallStyleName;
}

FToolMenuStringCommand UToolMenuEntryExtensions::MakeStringCommand(EToolMenuStringCommandType Type, FName CustomType, const FString& String)
{
	return FToolMenuStringCommand(Type, CustomType, String);
}

void UToolMenuEntryExtensions::BreakStringCommand(const FToolMenuStringCommand& InValue, EToolMenuStringCommandType& Type, FName& CustomType, FString& String)
{
	Type = InValue.Type;
	CustomType = InValue.CustomType;
	String = InValue.String;
}

FToolMenuOwner UToolMenuEntryExtensions::MakeToolMenuOwner(FName Name)
{
	return FToolMenuOwner(Name);
}

void UToolMenuEntryExtensions::BreakToolMenuOwner(const FToolMenuOwner& InValue, FName& Name)
{
	Name = InValue.TryGetName();
}

UObject* UToolMenuContextExtensions::FindByClass(const FToolMenuContext& Context, TSubclassOf<UObject> InClass)
{
	return Context.FindByClass(*InClass);
}

void UToolMenuContextExtensions::DebugLogContextObjectClasses(const FToolMenuContext& Context)
{
	const FString ContextObjectClasses = FString::JoinBy(Context.ContextObjects, TEXT(", "), [](const UObject* ContextObject) { 
		return ContextObject ? ContextObject->GetClass()->GetPathName() : TEXT("Null");
	});
	UE_LOGF(LogSlate, Warning, "DebugLogContextObjectClasses: %ls", *ContextObjectClasses);
}

void UToolMenuEntryExtensions::SetLabel(UPARAM(ref) FToolMenuEntry& Target, const FText& Label)
{
	Target.Label = Label;
}

FText UToolMenuEntryExtensions::GetLabel(const FToolMenuEntry& Target)
{
	return Target.Label.Get();
}

void UToolMenuEntryExtensions::SetToolTip(UPARAM(ref) FToolMenuEntry& Target, const FText& ToolTip)
{
	Target.ToolTip = ToolTip;
}

FText UToolMenuEntryExtensions::GetToolTip(const FToolMenuEntry& Target)
{
	return Target.ToolTip.Get();
}

void UToolMenuEntryExtensions::SetIcon(UPARAM(ref) FToolMenuEntry& Target, const FName StyleSetName, const FName StyleName, const FName SmallStyleName)
{
	if (SmallStyleName == NAME_None)
	{
		if (StyleSetName == NAME_None && StyleName == NAME_None)
		{
			Target.Icon = FSlateIcon();
		}
			
		Target.Icon = FSlateIcon(StyleSetName, StyleName);
	}

	Target.Icon = FSlateIcon(StyleSetName, StyleName, SmallStyleName);
}

void UToolMenuEntryExtensions::SetStringCommand(UPARAM(ref) FToolMenuEntry& Target, const EToolMenuStringCommandType Type, const FName CustomType, const FString& String)
{
	Target.ResetActions();
	Target.StringExecuteAction.Type = Type;
	Target.StringExecuteAction.CustomType = CustomType;
	Target.StringExecuteAction.String = String;
}

FToolMenuEntry UToolMenuEntryExtensions::InitMenuEntry(const FName InOwner, const FName InName, const FText& InLabel, const FText& InToolTip, const EToolMenuStringCommandType CommandType, const FName CustomCommandType, const FString& CommandString)
{
	FToolMenuEntry Entry(InOwner, InName, EMultiBlockType::MenuEntry);
	Entry.Label = InLabel;
	Entry.ToolTip = InToolTip;
	Entry.StringExecuteAction = FToolMenuStringCommand(CommandType, CustomCommandType, CommandString);
	return Entry;
}

void UToolMenuSectionExtensions::SetLabel(UPARAM(ref) FToolMenuSection& Section, const FText& Label)
{
	Section.Label = Label;
}

FText UToolMenuSectionExtensions::GetLabel(const FToolMenuSection& Section)
{
	return Section.Label.Get();
}

void UToolMenuSectionExtensions::AddEntry(UPARAM(ref) FToolMenuSection& Section, const FToolMenuEntry& Args)
{
	Section.AddEntry(Args);
}

void UToolMenuSectionExtensions::AddEntryObject(UPARAM(ref) FToolMenuSection& Section, UToolMenuEntryScript* InObject)
{
	Section.AddEntryObject(InObject);
}

