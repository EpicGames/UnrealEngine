// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenuMisc.h"
#include "ToolMenuEntryScript.h"
#include "Misc/Attribute.h"
#include "Templates/SubclassOf.h"

#include "ToolMenusBlueprintLibrary.generated.h"

UCLASS()
class UToolMenuContextExtensions : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod, DeterminesOutputType = "InClass"))
	static UObject* FindByClass(const FToolMenuContext& Context, TSubclassOf<UObject> InClass);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static void DebugLogContextObjectClasses(const FToolMenuContext& Context);
};

UCLASS()
class UToolMenuEntryExtensions : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintPure, Category = "Tool Menus", meta = (Keywords = "construct build", NativeMakeFunc))
	static FScriptSlateIcon MakeScriptSlateIcon(const FName StyleSetName, const FName StyleName, const FName SmallStyleName = NAME_None);

	UFUNCTION(BlueprintPure, Category = "Tool Menus", meta = (NativeBreakFunc))
	static void BreakScriptSlateIcon(const FScriptSlateIcon& InValue, FName& StyleSetName, FName& StyleName, FName& SmallStyleName);

	UFUNCTION(BlueprintPure, Category = "Tool Menus", meta = (Keywords = "construct build", NativeMakeFunc))
	static FToolMenuStringCommand MakeStringCommand(EToolMenuStringCommandType Type, FName CustomType, const FString& String);

	UFUNCTION(BlueprintPure, Category = "Tool Menus", meta = (NativeBreakFunc))
	static void BreakStringCommand(const FToolMenuStringCommand& InValue, EToolMenuStringCommandType& Type, FName& CustomType, FString& String);

	UFUNCTION(BlueprintPure, Category = "Tool Menus", meta = (Keywords = "construct build", NativeMakeFunc))
	static FToolMenuOwner MakeToolMenuOwner(FName Name);

	UFUNCTION(BlueprintPure, Category = "Tool Menus", meta = (NativeBreakFunc))
	static void BreakToolMenuOwner(const FToolMenuOwner& InValue, FName& Name);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static void SetLabel(UPARAM(ref) FToolMenuEntry& Target, const FText& Label);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static FText GetLabel(const FToolMenuEntry& Target);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static void SetToolTip(UPARAM(ref) FToolMenuEntry& Target, const FText& ToolTip);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static FText GetToolTip(const FToolMenuEntry& Target);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static void SetIcon(UPARAM(ref) FToolMenuEntry& Target, const FName StyleSetName, const FName StyleName = NAME_None, const FName SmallStyleName = NAME_None);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static void SetStringCommand(UPARAM(ref) FToolMenuEntry& Target, const EToolMenuStringCommandType Type, const FName CustomType, const FString& String);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	static FToolMenuEntry InitMenuEntry(const FName InOwner, const FName InName, const FText& InLabel, const FText& InToolTip, const EToolMenuStringCommandType CommandType, const FName CustomCommandType, const FString& CommandString);
};

UCLASS()
class UToolMenuSectionExtensions : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static void SetLabel(UPARAM(ref) FToolMenuSection& Section, const FText& Label);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static FText GetLabel(const FToolMenuSection& Section);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static void AddEntry(UPARAM(ref) FToolMenuSection& Section, const FToolMenuEntry& Args);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = (ScriptMethod))
	static void AddEntryObject(UPARAM(ref) FToolMenuSection& Section, UToolMenuEntryScript* InObject);
};
