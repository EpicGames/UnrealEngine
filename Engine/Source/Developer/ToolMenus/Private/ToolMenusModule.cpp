// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IToolMenusModule.h"


/**
 * Implements the Tool menus module.
 */
class FToolMenusModule
	: public IToolMenusModule
{
public:

	// IToolMenusModule interface

	virtual void StartupModule( ) override
	{
	}

	virtual void ShutdownModule( ) override
	{
	}

	// End IToolInterface interface
};


IMPLEMENT_MODULE(FToolMenusModule, ToolMenus)
