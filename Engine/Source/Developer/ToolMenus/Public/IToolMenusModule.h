// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IToolMenusModule : public IModuleInterface
{
public:

	/**
	 * Retrieve the module instance.
	 */
	static inline IToolMenusModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IToolMenusModule>("ToolMenus");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("ToolMenus");
	}
};

