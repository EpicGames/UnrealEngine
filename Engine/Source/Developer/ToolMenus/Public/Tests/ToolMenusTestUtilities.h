// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Misc/AutomationTest.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#define UE_API TOOLMENUS_API

namespace UE::ToolMenus
{
	enum class EExpectedOccurrence : uint8
	{
		Any,			// Zero or more
		All,			// Exact match
		ExactlyZero,	// Should not match at all
		ExactlyOne,
		OneOrMore
	};

	/**
	 * @note: that these parameters apply to the item type, not the children of that item (with some exceptions).
	 * When used for sections, it applies to Sections in general and not entries within that section.
	 */
	struct FMenuMatchParameters
	{
		/**
		 * If true, all expected entries/sections must exist in the actual menu.
		 * If false, only one or more expected items are expected to be found, and the test will pass even if additional items are expected.
		 */
		bool bActualHasAllExpectedItems = false;

		/** The expected entries/sections must be in the same order they are in the actual menu. */
		bool bActualHasExpectedOrder = true;

		/**
		 * If true, all of the actual items must be found in the expected menu.
		 * If false, only one or more actual items are expected to be found, and the test will pass even if additional items are found in the actual menu.
		 */
		bool bExpectedHasAllActualItems = false;

		/** Attempts to match children if any are expected, otherwise it skips them. */
		bool bMatchChildrenIfAnyExpected = true;
	};

	struct FMenuEntryBase
	{
		explicit FMenuEntryBase(const FName InName, const EExpectedOccurrence InOccurrence = EExpectedOccurrence::ExactlyOne)
			: Name(InName)
			, Occurrence(InOccurrence)
		{
		}

		virtual ~FMenuEntryBase() = default;

		FName Name;
		EExpectedOccurrence Occurrence = EExpectedOccurrence::ExactlyOne;

		virtual FName GetTypeName() = 0;
	};

	/** Represents a Named ToolMenuEntry, optionally checking the type. */
	struct FMenuEntry
		: FMenuEntryBase
		, public TSharedFromThis<FMenuEntry>
	{
		explicit FMenuEntry(const FName InName)
			: FMenuEntryBase(InName)
		{
		}

		virtual ~FMenuEntry() override = default;

		TOptional<FString> Label;
		TOptional<EMultiBlockType> Type = {};

		TSharedRef<FMenuEntry> WithLabel(const FString& InLabel)
		{
			Label = InLabel;
			return AsShared();
		}

		bool operator==(const FToolMenuEntry& InToolMenuEntry) const
		{
			return !Name.IsNone() && Name == InToolMenuEntry.Name
				&& (!Label.IsSet() || InToolMenuEntry.Label.Get().ToString().MatchesWildcard(Label.GetValue()))
				&& (!Type.IsSet() || Type.GetValue() == InToolMenuEntry.Type);
		}

		static UE_API FName TypeName;

		virtual FName GetTypeName() override
		{
			return TypeName;
		}
	};

	/** Represents the presence of any MenuEntry, used when the test is agnostic to what's placed/inserted into an extension point. */
	struct FMenuWildcardEntry : FMenuEntryBase
	{
		FMenuWildcardEntry()
			: FMenuEntryBase(NAME_None, EExpectedOccurrence::Any)
		{
		}

		virtual ~FMenuWildcardEntry() override = default;

		bool operator==(TConstArrayView<FToolMenuEntry> InToolMenuEntries) const
		{
			// @todo
			return true;
		}

		static UE_API FName TypeName;

		virtual FName GetTypeName() override
		{
			return TypeName;
		}
	};

	/** Represents a Named ToolMenuSection with zero or more (sorted) Menu Entries. */
	struct FMenuSection
		: public TSharedFromThis<FMenuSection>
	{
		explicit FMenuSection(const FName InName, const TArray<TSharedRef<FMenuEntryBase>>& InEntries = {})
			: Name(InName)
			, Entries(InEntries)
		{
		}

		FName Name;
		TOptional<FString> Label;
		TArray<TSharedRef<FMenuEntryBase>> Entries;

		TSharedRef<FMenuSection> WithLabel(const FString& InLabel)
		{
			Label = InLabel;
			return AsShared();
		}

		TSharedRef<FMenuSection> WithEntries(const TArray<TSharedRef<FMenuEntryBase>>& InEntries)
		{
			Entries = InEntries;
			return AsShared();
		}

		/** Only compare top-level item, ignore children (entries). */
		bool operator==(const FToolMenuSection& InToolMenuSection) const
		{
			return (!Name.IsNone() || Name == InToolMenuSection.Name)
				&& (!Label.IsSet() || InToolMenuSection.Label.Get().ToString().MatchesWildcard(Label.GetValue()));
		}
	};

	struct FMenu
		: FMenuEntryBase
		, public TSharedFromThis<FMenu>
	{
		explicit FMenu(const FName InName, const TArray<TSharedRef<FMenuSection>>& InSections = {})
			: FMenuEntryBase(InName)
			, Sections(InSections)
		{
		}

		virtual ~FMenu() override = default;

		TArray<TSharedRef<FMenuSection>> Sections;

        TSharedRef<FMenu> WithSections(const TArray<TSharedRef<FMenuSection>>& InSections)
        {
			Sections = InSections;
        	return AsShared();
        }

		/** Only compare top-level item, ignore children (sections). */
		bool operator==(const UToolMenu& InToolMenu) const
		{
        	return Name == InToolMenu.MenuName;
		}

		static UE_API FName TypeName;

		virtual FName GetTypeName() override
		{
			return TypeName;
		}
	};

	/**
	 * The factory functions below reduce verbosity when constructing Expected menu structures, and can be used like so:
	 * @example
	 * using namespace UE::ToolMenus;
     * const FMenu ExpectedMenuStructure(
     *		"SomeMenu",
     *		{
     *			Section("SomeSection",
     *			{
     *				Entry("SomeEntry"),
     *				Any(),
     *				Entry("AnotherEntry")
	 *			})
     * 		});
	 */

	inline TSharedRef<FMenuEntry> Entry()
	{
		return MakeShared<FMenuEntry>(NAME_None);
	}

	/** Creates a MenuEntry with a label to match, with wildcards. This will check if the actual entry being tested against matches the given label. */
	inline TSharedRef<FMenuEntry> Entry(const FName InName)
	{
		return MakeShared<FMenuEntry>(InName);
	}

	inline TSharedRef<FMenuEntry> Separator()
	{
		TSharedRef<FMenuEntry> Result = MakeShared<FMenuEntry>(NAME_None);
		Result->Type = EMultiBlockType::Separator;
		return Result;
	}

	inline TSharedRef<FMenuEntry> Any(const EExpectedOccurrence InOccurence = EExpectedOccurrence::Any)
	{
		TSharedRef<FMenuEntry> Result = MakeShared<FMenuEntry>(NAME_None);
		Result->Occurrence = InOccurence;
		return Result;
	}

	inline TSharedRef<FMenuSection> Section()
	{
		return MakeShared<FMenuSection>(NAME_None);
	}

	inline TSharedRef<FMenuSection> Section(const FName InName, const TArray<TSharedRef<FMenuEntryBase>>& InEntries = {})
	{
		return MakeShared<FMenuSection>(InName, InEntries);
	}

	inline TSharedRef<FMenu> SubMenu()
	{
		return MakeShared<FMenu>(NAME_None);
	}

	inline TSharedRef<FMenu> SubMenu(const FName InName, const TArray<TSharedRef<FMenuSection>>& InSections = {})
	{
		return MakeShared<FMenu>(InName, InSections);
	}

	/** @note: Tests will continue so long as they can - we don't return for the first failed case. */
	class FToolMenuAutomationTestAdapter
	{
	public:
		struct FToolMenuAutomationTestAdapterParameters
		{
			FToolMenuAutomationTestAdapterParameters()
				: SectionMatchParameters({ true, true, false, true })
				, EntryMatchParameters({ true, true, false, true })
			{

			}

			/** Note that regardless of match parameters, use of Any() will explicitly allow one or more unknown entries in the specified location. */
			explicit FToolMenuAutomationTestAdapterParameters(
				const FMenuMatchParameters& InSectionMatchParameters,
				const FMenuMatchParameters& InEntryMatchParameters)
				: SectionMatchParameters(InSectionMatchParameters)
				, EntryMatchParameters(InEntryMatchParameters)
			{
			}

			FMenuMatchParameters SectionMatchParameters;
			FMenuMatchParameters EntryMatchParameters;
		};

		explicit FToolMenuAutomationTestAdapter(FAutomationTestBase& InTestInstance, const FToolMenuAutomationTestAdapterParameters& InParameters = {})
			: TestInstance(InTestInstance)
			, Parameters(InParameters)
		{
		}

		TOOLMENUS_API bool Matches(const FMenuEntryBase* InExpectedEntry, const FToolMenuEntry& InActualEntry);
		TOOLMENUS_API bool Matches(const FMenuEntry& InExpectedEntry, const FToolMenuEntry& InActualEntry);
		TOOLMENUS_API bool Matches(const FMenuSection& InExpectedSection, const FToolMenuSection& InActualSection, bool bInTestChildren = true);
		TOOLMENUS_API bool Matches(const FMenu& InExpectedMenu, const UToolMenu& InActualMenu);

	private:
		template <typename ElementType>
		bool TestValidIndex(const FString& What, const int32 Index, const TArray<ElementType>& Array)
		{
			if (!Array.IsValidIndex(Index))
			{
				TestInstance.AddError(FString::Printf(TEXT("Expected %s at Index %d, but the array only contains %d elements."), *What, Index, Array.Num()), 1);
			}

			return true;
		}

	private:
		FAutomationTestBase& TestInstance;
		FToolMenuAutomationTestAdapterParameters Parameters;
	};
}

#undef UE_API
