// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolMenuOwner.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "ToolMenuContext.h"
#include "UObject/Object.h"
#include "Framework/MultiBox/ToolMenuBase.h"

#include "ToolMenu.generated.h"

#define UE_API TOOLMENUS_API

/**
 * Represents a menu in the ToolMenus system.
 *
 * An instance of this class is returned by basic APIs such as UToolMenus::RegisterMenu and UToolMenus::ExtendMenu.
 **/
UCLASS(MinimalAPI, BlueprintType)
class UToolMenu : public UToolMenuBase
{
	GENERATED_BODY()

public:

	UE_API UToolMenu();

	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void InitMenu(const FToolMenuOwner Owner, FName Name, FName Parent = NAME_None, EMultiBoxType Type = EMultiBoxType::Menu);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = ( ScriptName = "AddSection", DisplayName = "Add Section", AutoCreateRefTerm = "Label", AdvancedDisplay = "InsertName,InsertType" ))
	UE_API void AddSectionScript(
		const FName SectionName,
		const FText& Label = FText(),
		const FName InsertName = NAME_None,
		const EToolMenuInsertType InsertType = EToolMenuInsertType::Default,
		const EToolMenuSectionAlign Alignment = EToolMenuSectionAlign::Default
	);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = ( ScriptName = "AddDynamicSection", DisplayName = "Add Dynamic Section" ))
	UE_API void AddDynamicSectionScript(const FName SectionName, UToolMenuSectionDynamic* Object);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void AddMenuEntry(const FName SectionName, const FToolMenuEntry& Args);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void AddMenuEntryObject(UToolMenuEntryScript* InObject);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void RemoveMenuEntryObject(UToolMenuEntryScript* InObject);

	UFUNCTION(BlueprintCallable, Category = "Tool Menus", meta = ( ScriptName = "AddSubMenu", AutoCreateRefTerm = "Label,ToolTip" ))
	UE_API UToolMenu* AddSubMenuScript(const FName Owner, const FName SectionName, const FName Name, const FText& Label, const FText& ToolTip = FText());

	UE_API UToolMenu* AddSubMenu(const FToolMenuOwner Owner, const FName SectionName, const FName Name, const FText& Label, const FText& ToolTip = FText());

	UE_API void RemoveSection(const FName SectionName);

	/**
	 * Add a section to the menu.
	 * When adding sections while registering a menu with UToolMenus::Get()->RegisterMenu(), always use this function.
	 * This will ensure that sections have the right label and are presented in the intended order
	 * regardless of when menu extensions are performed. 
	 * @param SectionName The name of the section.
	 * @param InLabel The label to set on the section.
	 * @param InPosition The position of the section within its parent menu.
	 * @return A reference to the created section.
	 */
	UE_API FToolMenuSection& AddSection(const FName SectionName, const TAttribute< FText >& InLabel = TAttribute<FText>(), const FToolMenuInsert InPosition = FToolMenuInsert());

	/**
	 * Build a section in a callback.
	 *
	 * The callback is passed a sandboxed empty menu to add as many sections as desired. Sections are merged into the final menu afterwards.
	 *
	 * @param SectionName The FName of the section the callback will build.
	 * @param InConstruct The callback that will be run to build the menu as the user opens it.
	 * @param InPosition The position of the section within its parent menu.
	 * @return A reference to the created section.
	 */
	UE_API FToolMenuSection& AddDynamicSection(const FName SectionName, const FNewSectionConstructChoice& InConstruct, const FToolMenuInsert InPosition = FToolMenuInsert());

	/**
	 * Finds an existing section using the provided FName.
	 *
	 * @param SectionName The section name to find. Provide NAME_None to access the unnamed section of this menu.
	 * @return A pointer to the found section or nullptr if none was found.
	 */
	UE_API FToolMenuSection* FindSection(const FName SectionName);

	/**
	 * Finds an existing section using the provided FName or, if not found, creates a new one and assigns it that FName.
	 *
	 * @param SectionName The section name to find or add. Provide NAME_None to access the unnamed section of this menu.
	 * @return A reference to the found or newly created section.
	 */
	UE_API FToolMenuSection& FindOrAddSection(const FName SectionName);

	/**
	 * Finds an existing section using the provided FName or, if not found, creates a new one and assigns it that FName.
	 *
	 * In case the the section did not exist and a new one was created, then the provided label and optional
	 * position is also set on the newly created section.
	 *
	 * @param SectionName The section name to find or add. Provide NAME_None to access the unnamed section of this menu.
	 * @param InLabel The label to set on the section if a new one is created. If an existing section is returned this label is ignored.
	 * @param InPosition The position to set on the section if a new one is created. If an existing section is returned this position is ignored.
	 * @return A reference to the found or newly created section.
	 */
	UE_API FToolMenuSection& FindOrAddSection(const FName SectionName, const TAttribute< FText >& InLabel, const FToolMenuInsert InPosition = FToolMenuInsert());

	FName GetMenuName() const { return MenuName; }

	bool IsRegistered() const { return bRegistered; }

	/** returns array [Menu, Menu.SubMenuA, Menu.SubMenuB] for Menu.SubMenuB.SubMenuB */
	UE_API TArray<const UToolMenu*> GetSubMenuChain() const;

	/** returns "SubMenuC.SubMenuD" for menu "ModuleA.MenuB.SubMenuC.SubMenuD" */
	UE_API FString GetSubMenuNamePath() const;

	/* Set support for extenders */
	UE_API void SetExtendersEnabled(bool bEnabled);

	//~ Begin UToolMenuBase Interface
	UE_API virtual bool IsEditing() const override;
	UE_API virtual FName GetSectionName(const FName InEntryName) const override;
	UE_API virtual bool ContainsSection(const FName InName) const override;
	UE_API virtual bool ContainsEntry(const FName InName) const override;
	UE_API virtual FCustomizedToolMenu* FindMenuCustomization() const override;
	UE_API virtual FCustomizedToolMenu* AddMenuCustomization() const override;
	UE_API virtual FCustomizedToolMenuHierarchy GetMenuCustomizationHierarchy() const override;
	UE_API virtual FToolMenuProfile* FindMenuProfile(const FName& ProfileName) const override;
	UE_API virtual FToolMenuProfile* AddMenuProfile(const FName& ProfileName) const override;
	UE_API virtual FToolMenuProfileHierarchy GetMenuProfileHierarchy(const FName& ProfileName) const override;
	UE_API virtual void UpdateMenuCustomizationFromMultibox(const TSharedRef<const FMultiBox>& InMultiBox) override;
	UE_API virtual void OnMenuDestroyed() override;
	//~ End UToolMenuBase Interface

	UE_API TArray<FName> GetMenuHierarchyNames(bool bIncludeSubMenuRoot) const;
	void SetMaxHeight(uint32 InMaxHeight)
	{
		MaxHeight = InMaxHeight;
	}

	template <typename TContextType>
	TContextType* FindContext() const
	{
		return Context.FindContext<TContextType>();
	}


	UE_API FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	UE_API const ISlateStyle* GetStyleSet() const;
	UE_API void SetStyleSet(const ISlateStyle* InStyleSet);

	friend class UToolMenus;

private:

	UE_API void InitGeneratedCopy(const UToolMenu* Source, const FName InMenuName, const FToolMenuContext* InContext = nullptr);

	UE_API bool FindEntry(const FName EntryName, int32& SectionIndex, int32& EntryIndex) const;

	UE_API FToolMenuEntry* FindEntry(const FName EntryName);

	UE_API const FToolMenuEntry* FindEntry(const FName EntryName) const;

	UE_API int32 IndexOfSection(const FName SectionName) const;

	UE_API int32 FindInsertIndex(const FToolMenuSection& InSection) const;

	UE_API bool IsRegistering() const;

	UE_API void Empty();

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName MenuName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName MenuParent;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName StyleName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName TutorialHighlightName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	EMultiBoxType MenuType;

	UPROPERTY(Transient)
	bool bShouldCleanupContextOnDestroy;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	bool bShouldCloseWindowAfterMenuSelection;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	bool bCloseSelfOnly;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	bool bSearchable;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ToolBar")
	bool bToolBarIsFocusable;
	
	/** Whether different sections should have separators automatically added between them. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ToolBar")
	bool bSeparateSections = true;

	/** Whether to (ever) allow the wrap button/overflow menu. Defauls to the style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ToolBar")
	TOptional<bool> bAllowToolBarWrapButton;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ToolBar")
	bool bToolBarForceSmallIcons;

	/** Prevent menu from being customized */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ToolBar")
	bool bPreventCustomization;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FToolMenuOwner MenuOwner;
	
	UPROPERTY()
	FToolMenuContext Context;

	UPROPERTY()
	TArray<FToolMenuSection> Sections;

	UPROPERTY()
	TObjectPtr<const UToolMenu> SubMenuParent;

	UPROPERTY()
	FName SubMenuSourceEntryName;

	FMultiBox::FOnModifyBlockWidgetAfterMake ModifyBlockWidgetAfterMake;

private:

	bool bRegistered;
	bool bIsRegistering;
	bool bExtendersEnabled;

	FName StyleSetName;

	uint32 MaxHeight;
};

#undef UE_API
