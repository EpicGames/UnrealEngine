// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/SortedMap.h"

#include "ToolMenuContext.generated.h"

#define UE_API TOOLMENUS_API

struct FUIAction;
class FUICommandInfo;
class FUICommandList;
class FTabManager;
class FExtender;

UCLASS(MinimalAPI, BlueprintType, Abstract)
class UToolMenuContextBase : public UObject
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class USlateTabManagerContext : public UToolMenuContextBase
{
	GENERATED_BODY()
public:

	TWeakPtr<FTabManager> TabManager;
};

USTRUCT(BlueprintType)
struct FToolMenuContext
{
	GENERATED_BODY()
public:

	using FContextObjectCleanup = TFunction<void(UObject*)>;
	using FContextCleanup = TFunction<void()>;

	FToolMenuContext() = default;
	UE_API FToolMenuContext(UObject* InContext);
	UE_API FToolMenuContext(UObject* InContext, FContextObjectCleanup&& InCleanup);
	UE_API FToolMenuContext(TSharedPtr<FUICommandList> InCommandList, TSharedPtr<FExtender> InExtender = TSharedPtr<FExtender>(), UObject* InContext = nullptr);

	template <typename TContextType>
	TContextType* FindContext() const
	{
		for (UObject* Object : ContextObjects)
		{
			if (TContextType* Result = Cast<TContextType>(Object))
			{
				return Result;
			}
		}

		return nullptr;
	}

	UE_API UObject* FindByClass(UClass* InClass) const;

	UE_API void AppendCommandList(const TSharedRef<FUICommandList>& InCommandList);
	UE_API void AppendCommandList(const TSharedPtr<FUICommandList>& InCommandList);
	UE_API const FUIAction* GetActionForCommand(TSharedPtr<const FUICommandInfo> Command, TSharedPtr<const FUICommandList>& OutCommandList) const;
	UE_API const FUIAction* GetActionForCommand(TSharedPtr<const FUICommandInfo> Command) const;

	UE_API void AddExtender(const TSharedPtr<FExtender>& InExtender);
	UE_API TSharedPtr<FExtender> GetAllExtenders() const;
	UE_API void ResetExtenders();

	UE_API void AppendObjects(const TArray<UObject*>& InObjects);
	UE_API void AddObject(UObject* InObject);
	UE_API void AddObject(UObject* InObject, FContextObjectCleanup&& InCleanup);

	UE_API void AddCleanup(FContextCleanup&& InCleanup);

	UE_API void CleanupObjects();

	friend class UToolMenus;
	friend class UToolMenu;
	friend struct FToolMenuEntry;
	friend class UToolMenuContextExtensions;

	bool IsEditing() const { return bIsEditing; }
	void SetIsEditing(bool InIsEditing) { bIsEditing = InIsEditing; }

private:

	UE_API void Empty();

	bool bIsEditing = false;

	UPROPERTY()
	TArray<TObjectPtr<UObject>> ContextObjects;

	TSortedMap<TObjectPtr<UObject>, FContextObjectCleanup> ContextObjectCleanupFuncs;

	TArray<FContextCleanup> ContextCleanupFuncs;

	TArray<TSharedPtr<FUICommandList>> CommandLists;

	TSharedPtr<FUICommandList> CommandList;

	TArray<TSharedPtr<FExtender>> Extenders;
};

#undef UE_API
