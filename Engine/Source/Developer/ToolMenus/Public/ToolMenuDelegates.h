// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolMenuContext.h"
#include "ToolMenuMisc.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "ToolMenuDelegates.generated.h"

#define UE_API TOOLMENUS_API

class UToolMenu;
struct FToolMenuEntry;
struct FToolMenuSection;
struct FToolMenuCustomWidgetContext;

DECLARE_DELEGATE_OneParam(FNewToolMenuSectionDelegate, FToolMenuSection&);
DECLARE_DELEGATE_OneParam(FNewToolMenuDelegate, UToolMenu*);
DECLARE_DELEGATE_TwoParams(FNewToolMenuDelegateLegacy, class FMenuBuilder&, UToolMenu*);
DECLARE_DELEGATE_TwoParams(FNewToolBarDelegateLegacy, class FToolBarBuilder&, UToolMenu*);
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FNewToolMenuWidget, const FToolMenuContext&);
DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SWidget>, FNewToolMenuCustomWidget, const FToolMenuContext&, const FToolMenuCustomWidgetContext&);

DECLARE_DELEGATE_OneParam(FToolMenuExecuteAction, const FToolMenuContext&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FToolMenuCanExecuteAction, const FToolMenuContext&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FToolMenuIsActionChecked, const FToolMenuContext&);
DECLARE_DELEGATE_RetVal_OneParam(ECheckBoxState, FToolMenuGetActionCheckState, const FToolMenuContext&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FToolMenuIsActionButtonVisible, const FToolMenuContext&);

DECLARE_DELEGATE_RetVal_ThreeParams(bool, FToolMenuSectionSorter, const FToolMenuEntry&, const FToolMenuEntry&, const FToolMenuContext&);

DECLARE_DELEGATE_TwoParams(FToolMenuExecuteString, const FString&, const FToolMenuContext&);

DECLARE_DYNAMIC_DELEGATE_OneParam(FToolMenuDynamicExecuteAction, const FToolMenuContext&, Context);
DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FToolMenuDynamicCanExecuteAction, const FToolMenuContext&, Context);
DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FToolMenuDynamicIsActionChecked, const FToolMenuContext&, Context);
DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(ECheckBoxState, FToolMenuDynamicGetActionCheckState, const FToolMenuContext&, Context);
DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FToolMenuDynamicIsActionButtonVisible, const FToolMenuContext&, Context);

struct FToolUIAction
{
	FToolUIAction() {}
	FToolUIAction(const FToolMenuExecuteAction& InExecuteAction) : ExecuteAction(InExecuteAction) {}
	FToolUIAction(const FToolMenuExecuteAction& InExecuteAction, const FToolMenuCanExecuteAction& InCanExecuteAction, const FToolMenuGetActionCheckState& InGetActionCheckState)
		: ExecuteAction(InExecuteAction)
		, CanExecuteAction(InCanExecuteAction)
		, GetActionCheckState(InGetActionCheckState)
	{}
	
	FToolUIAction(const FToolMenuExecuteAction& InExecuteAction, const FToolMenuGetActionCheckState& InGetActionCheckState)
		: ExecuteAction(InExecuteAction)
		, GetActionCheckState(InGetActionCheckState)
	{}

	FToolMenuExecuteAction ExecuteAction;
	FToolMenuCanExecuteAction CanExecuteAction;
	FToolMenuGetActionCheckState GetActionCheckState;
	FToolMenuIsActionButtonVisible IsActionVisibleDelegate;
};

USTRUCT(BlueprintType)
struct FToolDynamicUIAction
{
	GENERATED_BODY()

	FToolDynamicUIAction() {}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FToolMenuDynamicExecuteAction ExecuteAction;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FToolMenuDynamicCanExecuteAction CanExecuteAction;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FToolMenuDynamicGetActionCheckState GetActionCheckState;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FToolMenuDynamicIsActionButtonVisible IsActionVisibleDelegate;
};

struct FNewToolMenuChoice
{
	FNewToolMenuChoice() {}
	FNewToolMenuChoice(const FOnGetContent& InOnGetContent) : OnGetContent(InOnGetContent) {}
	FNewToolMenuChoice(const FNewToolMenuWidget& InNewToolMenuWidget) : NewToolMenuWidget(InNewToolMenuWidget) {}
	FNewToolMenuChoice(const FNewToolMenuDelegate& InNewToolMenu) : NewToolMenu(InNewToolMenu) {}
	FNewToolMenuChoice(const FNewMenuDelegate& InNewMenuLegacy) : NewMenuLegacy(InNewMenuLegacy) {}

	FOnGetContent OnGetContent;
	FNewToolMenuWidget NewToolMenuWidget;
	FNewToolMenuDelegate NewToolMenu;
	FNewMenuDelegate NewMenuLegacy;
};

struct FToolUIActionChoice
{
public:
	FToolUIActionChoice() {}
	FToolUIActionChoice(const FUIAction& InAction) : Action(InAction) {}
	FToolUIActionChoice(const FExecuteAction& InExecuteAction) : Action(InExecuteAction) {}
	FToolUIActionChoice(const FToolUIAction& InAction) : ToolAction(InAction) {}
	FToolUIActionChoice(const FToolDynamicUIAction& InAction) : DynamicToolAction(InAction) {}
	FToolUIActionChoice(const FToolMenuExecuteAction& InExecuteAction) : ToolAction(InExecuteAction) {}
	UE_API FToolUIActionChoice(const TSharedPtr< const FUICommandInfo >& InCommand, const FUICommandList& InCommandList);

	const FUIAction* GetUIAction() const
	{
		return Action.IsSet() ? &Action.GetValue() : nullptr;
	}

	const FToolUIAction* GetToolUIAction() const
	{
		return ToolAction.IsSet() ? &ToolAction.GetValue() : nullptr;
	}

	const FToolDynamicUIAction* GetToolDynamicUIAction() const
	{
		return DynamicToolAction.IsSet() ? &DynamicToolAction.GetValue() : nullptr;
	}

private:
	TOptional<FUIAction> Action;
	TOptional<FToolUIAction> ToolAction;
	TOptional<FToolDynamicUIAction> DynamicToolAction;
};

struct FNewSectionConstructChoice
{
	FNewSectionConstructChoice() {};
	FNewSectionConstructChoice(const FNewToolMenuDelegate& InNewToolMenuDelegate) : NewToolMenuDelegate(InNewToolMenuDelegate) {}
	FNewSectionConstructChoice(const FNewToolMenuDelegateLegacy& InNewToolMenuDelegateLegacy) : NewToolMenuDelegateLegacy(InNewToolMenuDelegateLegacy) {}
	FNewSectionConstructChoice(const FNewToolBarDelegateLegacy& InNewToolBarDelegateLegacy) : NewToolBarDelegateLegacy(InNewToolBarDelegateLegacy) {}

	FNewToolMenuDelegate NewToolMenuDelegate;
	FNewToolMenuDelegateLegacy NewToolMenuDelegateLegacy;
	FNewToolBarDelegateLegacy NewToolBarDelegateLegacy;

	bool IsBound() const { return NewToolMenuDelegate.IsBound() || NewToolMenuDelegateLegacy.IsBound() || NewToolBarDelegateLegacy.IsBound(); }
};

#undef UE_API
