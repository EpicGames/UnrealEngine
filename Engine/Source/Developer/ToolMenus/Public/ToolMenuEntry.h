// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolMenuOwner.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuMisc.h"

#include "Misc/Attribute.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/Commands/UICommandInfo.h"
#include "UObject/TextProperty.h"
#include "ToolMenuEntry.generated.h"

#define UE_API TOOLMENUS_API

class UToolMenuEntryScript;
struct FKeyEvent;

/** A StyleSet, StyleName pair */
struct FToolMenuEntryStyle
{
	/** The StyleSet used to create the calling widget. Default is auto-calculated or derived from elsewhere */
	TOptional<const class ISlateStyle*> StyleSet;

	/** The name of the style (within the StyleSet) to use to create the calling widget. Default is auto-calculated or derived from elsewhere */
	TOptional<FName> StyleName;
};

struct FToolMenuEntrySubMenuData
{
public:
	FToolMenuEntrySubMenuData() :
		bIsSubMenu(false),
		bOpenSubMenuOnClick(false),
		bAutoCollapse(false)
	{
	}

	bool bIsSubMenu;
	bool bOpenSubMenuOnClick;
	/** Entry placed into the parent's menu when there is only one entry */
	bool bAutoCollapse;
	FNewToolMenuChoice ConstructMenu;

	/** Optionally specify to override the default styling */
	FToolMenuEntryStyle Style;
};

struct FToolMenuEntryOptionsDropdownData
{
	FNewToolMenuChoice MenuContentGenerator;
	TAttribute<FText> ToolTip;
	FUIAction Action;
};

struct FToolMenuEntryToolBarData
{
public:
	/** Optional override label to use when the entry appears in a toolbar. */
	TAttribute<FText> LabelOverride;

	/** Optional style override used when this entry appears in a toolbar. */
	FName StyleNameOverride = NAME_None;
	
	/** When set, entries with the same name will be grouped together. */
	FName BlockGroupName = NAME_None;

	/** Delegate that generates a widget for this combo button's menu content. Called when the menu is summoned. */
	FNewToolMenuChoice ComboButtonContextMenuGenerator;

	/** Legacy delegate that generates a widget for this combo button's menu content. Called when the menu is summoned. */
	FNewToolBarDelegateLegacy ConstructLegacy;

	TSharedPtr<FToolMenuEntryOptionsDropdownData> OptionsDropdownData;

	bool bSimpleComboBox = false;

	/** Whether ToolBar will have Focusable buttons. */
	bool bIsFocusable = false;

	/** Whether this toolbar should always use small icons, regardless of the current settings. */
	bool bForceSmallIcons = false;

	/** Optional override placement to customize menu placement once opened via e.g. a toolbar menu button */
	TAttribute<EMenuPlacement> PlacementOverride;

	/** Various Resize parameters and overrides, will take precedent over those stored in FToolMenuEntryWidgetData for ToolBar entries. */
	FMenuEntryResizeParams ResizeParams;

	/** Optional action override to use when the entry appears in a toolbar */
	TOptional<FToolUIAction> ActionOverride;
};


struct FToolMenuEntryWidgetData
{
public:
	FToolMenuEntryWidgetData() :
		bNoIndent(false),
		bSearchable(false),
		bNoPadding(false)
	{
	}

	/** Remove the padding from the left of the widget that lines it up with other menu items */
	bool bNoIndent;

	/** If true, widget will be searchable */
	bool bSearchable;

	/** If true, no padding will be added */
	bool bNoPadding;

	/** Various Style parameters and overrides */
	FMenuEntryStyleParams StyleParams;

	/** Various Resize parameters and overrides */
	FMenuEntryResizeParams ResizeParams;
};

struct FToolMenuCustomWidgetContext
{
	// The style used by the menu creating the widget
	const class ISlateStyle* StyleSet = nullptr;

	// The name of the style used by the menu creating the widget
	FName StyleName;
};

/**
 * A convenience wrapper for multiple ways of delivering a visibility value.
 * Use by setting directly to TAttribute<EVisibility>, TAttribute<bool>, or FIsActionButtonVisible.
 * Can also assign directly to a lambda, e.g `Choice = []{ return TrueOrFalse(); };`
 */
struct FToolMenuVisibilityChoice
{
	UE_API FToolMenuVisibilityChoice();
	UE_API FToolMenuVisibilityChoice(const FToolMenuVisibilityChoice& Other);
	UE_API FToolMenuVisibilityChoice(const TAttribute<EVisibility>& InVisibility);
	UE_API FToolMenuVisibilityChoice(const TAttribute<bool>& InIsVisible);
	UE_API FToolMenuVisibilityChoice(const FIsActionButtonVisible& InIsActionButtonVisible);
	
	UE_API TAttribute<EVisibility> ToVisibilityAttribute() const;
	operator TAttribute<EVisibility>() const { return ToVisibilityAttribute(); }
	
	UE_API FToolMenuVisibilityChoice& operator=(TFunction<EVisibility()> VisibilityFunc);
	UE_API FToolMenuVisibilityChoice& operator=(TFunction<bool()> IsVisibleFunc);
	
	UE_API bool IsSet() const;
	UE_API EVisibility Get() const;
	
private:
	TVariant<TAttribute<EVisibility>, TAttribute<bool>, FIsActionButtonVisible> Value;
};

/**
 * Represents entries in menus such as buttons, checkboxes, and sub-menus.
 *
 * Many entries are created for you via the methods of FToolMenuSection, such as FToolMenuSection::AddMenuEntry.
 */
USTRUCT(BlueprintType)
struct FToolMenuEntry
{
	GENERATED_BODY()

	UE_API FToolMenuEntry();
	UE_API FToolMenuEntry(const FToolMenuOwner InOwner, const FName InName, EMultiBlockType InType);

	UE_API FToolMenuEntry(const FToolMenuEntry&);
	UE_API FToolMenuEntry(FToolMenuEntry&&);

	UE_API FToolMenuEntry& operator=(const FToolMenuEntry&);
	UE_API FToolMenuEntry& operator=(FToolMenuEntry&&);

	static UE_API FToolMenuEntry InitMenuEntry(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon, const FToolUIActionChoice& InAction, const EUserInterfaceActionType UserInterfaceActionType = EUserInterfaceActionType::Button, const FName InTutorialHighlightName = NAME_None, const TOptional<FText>& InInputBindingOverride = TOptional<FText>());
	static UE_API FToolMenuEntry InitMenuEntry(const TSharedPtr< const FUICommandInfo >& InCommand, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(), const FName InTutorialHighlightName = NAME_None, const TOptional<FName> InNameOverride = TOptional<FName>());
	static UE_API FToolMenuEntry InitMenuEntry(const FName InNameOverride, const TSharedPtr< const FUICommandInfo >& InCommand, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(), const FName InTutorialHighlightName = NAME_None);
	static UE_API FToolMenuEntry InitMenuEntryWithCommandList(const TSharedPtr<const FUICommandInfo>& InCommand, const TSharedPtr<const FUICommandList>& InCommandList, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(), const FName InTutorialHighlightName = NAME_None, const TOptional<FName> InNameOverride = TOptional<FName>());
	static UE_API FToolMenuEntry InitMenuEntry(const FName InName, const FToolUIActionChoice& InAction, const TSharedRef<SWidget>& Widget);
	static UE_API FToolMenuEntry InitDynamicEntry(const FName InName, const FNewToolMenuSectionDelegate& InConstruct);

	static UE_API FToolMenuEntry InitSubMenu(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewToolMenuChoice& InMakeMenu, const FToolUIActionChoice& InAction, const EUserInterfaceActionType InUserInterfaceActionType, bool bInOpenSubMenuOnClick = false, const TAttribute<FSlateIcon>& InIcon = TAttribute<FSlateIcon>(), const bool bShouldCloseWindowAfterMenuSelection = true);
	static UE_API FToolMenuEntry InitSubMenu(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewToolMenuChoice& InMakeMenu, bool bInOpenSubMenuOnClick = false, const TAttribute<FSlateIcon>& InIcon = TAttribute<FSlateIcon>(), const bool bShouldCloseWindowAfterMenuSelection = true, const FName InTutorialHighlightName = NAME_None);
	static UE_API FToolMenuEntry InitSubMenu(const FName InName, const FToolUIActionChoice& InAction, const TSharedRef<SWidget>& InWidget, const FNewToolMenuChoice& InMakeMenu, bool bShouldCloseWindowAfterMenuSelection = true);

	static UE_API FToolMenuEntry InitToolBarButton(const FName InName, const FToolUIActionChoice& InAction, const TAttribute<FText>& InLabel = TAttribute<FText>(), const TAttribute<FText>& InToolTip = TAttribute<FText>(), const TAttribute<FSlateIcon>& InIcon = TAttribute<FSlateIcon>(), const EUserInterfaceActionType UserInterfaceActionType = EUserInterfaceActionType::Button, FName InTutorialHighlightName = NAME_None);
	static UE_API FToolMenuEntry InitToolBarButton(const TSharedPtr< const FUICommandInfo >& InCommand, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(), FName InTutorialHighlightName = NAME_None, const TOptional<FName> InNameOverride = TOptional<FName>());
	static UE_API FToolMenuEntry InitComboButton(const FName InName, const FToolUIActionChoice& InAction, const FNewToolMenuChoice& InMenuContentGenerator, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(), bool bInSimpleComboBox = false, FName InTutorialHighlightName = NAME_None);

	static UE_API FToolMenuEntry InitSeparator(const FName InName);

	static UE_API FToolMenuEntry InitWidget(const FName InName, const TSharedRef<SWidget>& InWidget, const FText& Label, bool bNoIndent = false, bool bSearchable = true, bool bNoPadding = false, const FText& InToolTipText = FText());

	bool IsSubMenu() const { return SubMenuData.bIsSubMenu; }

	bool IsConstructLegacy() const { return ConstructLegacy.IsBound(); }

	/**
	 * Get the checked state of this entry by calling underlying commands and delegates.
	 *
	 * @return: the checked state of this entry. */
	UE_API ECheckBoxState GetCheckState(const FToolMenuContext& InContext) const;

	UE_API const FUIAction* GetActionForCommand(const FToolMenuContext& InContext, TSharedPtr<const FUICommandList>& OutCommandList) const;

	UE_API void SetCommandList(const TSharedPtr<const FUICommandList>& InCommandList);

	UE_API void AddKeybindFromCommand(const TSharedPtr< const FUICommandInfo >& InCommand);

	UE_API bool IsCommandKeybindOnly() const;
	UE_API bool CommandAcceptsInput(const FKeyEvent& InKeyEvent) const;
	UE_API bool TryExecuteToolUIAction(const FToolMenuContext& InContext);
	friend struct FToolMenuSection;
	friend class UToolMenuEntryScript;

	/**
	 * Show this menu entry in the top-level toolbar section of a toolbar.
	 *
	 * Entries of a toolbar submenu can be raised to the top-level of the toolbar. Such top-level entries appear in the
	 * toolbar to the right of the submenu they belong to.
	 *
	 * This flag only effects entries within submenus of toolbar-type ToolMenus.
	 *
	 * THIS AFFECTS STYLING. When an entry is raised to the top level of a toolbar, the ".Raised" suffix is added to
	 * the style name that would otherwise have been applied.
	 *
	 * @param InTopLevel True shows the entry in the top-level next to its submenu, false (default) only displays it in
	 * the submenu itself. Pass a delegate to drive the top-level state from code.
	 */
	UE_API void SetShowInToolbarTopLevel(TAttribute<bool> InTopLevel);

private:

	UE_API void SetCommand(const TSharedPtr< const FUICommandInfo >& InCommand, TOptional<FName> InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon);

	UE_API void ResetActions();

	UE_API bool IsScriptObjectDynamicConstruct() const;

	UE_API bool IsNonLegacyDynamicConstruct() const;

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FToolMenuOwner Owner;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	EMultiBlockType Type;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	EUserInterfaceActionType UserInterfaceActionType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName TutorialHighlightName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FToolMenuInsert InsertPosition;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	bool bShouldCloseWindowAfterMenuSelection;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	TObjectPtr<UToolMenuEntryScript> ScriptObject;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName StyleNameOverride;

	FToolMenuEntrySubMenuData SubMenuData;

	FToolMenuEntryToolBarData ToolBarData;

	FToolMenuEntryWidgetData WidgetData;

	/** Optional delegate that returns a widget to use as this menu entry */
	FNewToolMenuCustomWidget MakeCustomWidget;

	TAttribute<FText> Label;
	TAttribute<FText> ToolTip;
	TAttribute<FSlateIcon> Icon;
	TAttribute<FText> InputBindingLabel;

	FToolMenuVisibilityChoice Visibility;
	
private:

	friend class UToolMenus;
	friend class UToolMenuEntryExtensions;
	friend class FPopulateMenuBuilderWithToolMenuEntry;

	FToolUIActionChoice Action;

	FToolMenuStringCommand StringExecuteAction;

	TSharedPtr< const FUICommandInfo > Command;
	TSharedPtr< const FUICommandList > CommandList;

	FNewToolMenuSectionDelegate Construct;
	FNewToolMenuDelegateLegacy ConstructLegacy;

	bool bAddedDuringRegister;

	UPROPERTY()
	bool bCommandIsKeybindOnly;

	TAttribute<bool> ShowInToolbarTopLevel;
};

#undef UE_API
