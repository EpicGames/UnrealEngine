// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/TextProperty.h"
#include "Textures/SlateIcon.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Styling/SlateTypes.h"
#include "UObject/UObjectThreadContext.h"

#include "ToolMenuMisc.h"
#include "ToolMenuContext.h"
#include "ToolMenuSection.h"

#include "ToolMenuEntryScript.generated.h"

#define UE_API TOOLMENUS_API

struct FToolMenuEntry;
struct FToolMenuSection;

USTRUCT(BlueprintType, meta=(HasNativeBreak="/Script/ToolMenus.ToolMenuEntryExtensions.BreakScriptSlateIcon", HasNativeMake="/Script/ToolMenus.ToolMenuEntryExtensions.MakeScriptSlateIcon"))
struct FScriptSlateIcon
{
	GENERATED_BODY()

public:
	UE_API FScriptSlateIcon();
	UE_API FScriptSlateIcon(const FName InStyleSetName, const FName InStyleName);
	UE_API FScriptSlateIcon(const FName InStyleSetName, const FName InStyleName, const FName InSmallStyleName);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName StyleSetName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName StyleName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName SmallStyleName;

	operator FSlateIcon() const { return GetSlateIcon(); }

	UE_API FSlateIcon GetSlateIcon() const;
};

USTRUCT(BlueprintType)
struct FToolMenuEntryScriptDataAdvanced
{
	GENERATED_BODY()

public:

	UE_API FToolMenuEntryScriptDataAdvanced();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Advanced")
	FName TutorialHighlight;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Advanced")
	EMultiBlockType EntryType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Advanced")
	EUserInterfaceActionType UserInterfaceActionType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Advanced")
	FName StyleNameOverride;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SubMenu")
	bool bIsSubMenu;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SubMenu")
	bool bOpenSubMenuOnClick;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Advanced")
	bool bShouldCloseWindowAfterMenuSelection;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ToolBar")
	bool bSimpleComboBox;
};

USTRUCT(BlueprintType)
struct FToolMenuEntryScriptData
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Identity")
	FName Menu;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Identity")
	FName Section;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Identity")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	FText Label;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	FText ToolTip;

	UPROPERTY(EditAnywhere,  BlueprintReadWrite, Category = "Appearance")
	FScriptSlateIcon Icon;

	// Optional identifier used for unregistering a group of menu items
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Advanced")
	FName OwnerName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Identity")
	FToolMenuInsert InsertPosition;

	UPROPERTY(EditAnywhere,  BlueprintReadWrite, Category = "Advanced")
	FToolMenuEntryScriptDataAdvanced Advanced;
};

UCLASS(MinimalAPI, Blueprintable, abstract)
class UToolMenuEntryScript : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintImplementableEvent, Category = "Action")
	UE_API void Execute(const FToolMenuContext& Context);

	UFUNCTION(BlueprintNativeEvent, Category = "Advanced")
	UE_API bool CanExecute(const FToolMenuContext& Context) const;
	virtual bool CanExecute_Implementation(const FToolMenuContext& Context) const { return true; }

	UFUNCTION(BlueprintNativeEvent, Category = "Advanced")
	UE_API ECheckBoxState GetCheckState(const FToolMenuContext& Context) const;
	virtual ECheckBoxState GetCheckState_Implementation(const FToolMenuContext& Context) const { return ECheckBoxState::Undetermined; }

	UFUNCTION(BlueprintNativeEvent, Category = "Advanced")
	UE_API bool IsVisible(const FToolMenuContext& Context) const;
	virtual bool IsVisible_Implementation(const FToolMenuContext& Context) const { return true; }

	UFUNCTION(BlueprintNativeEvent, Category = "Advanced")
	UE_API FText GetLabel(const FToolMenuContext& Context) const;
	virtual FText GetLabel_Implementation(const FToolMenuContext& Context) const { return Data.Label; }

	UFUNCTION(BlueprintNativeEvent, Category = "Advanced")
	UE_API FText GetToolTip(const FToolMenuContext& Context) const;
	virtual FText GetToolTip_Implementation(const FToolMenuContext& Context) const { return Data.ToolTip; }

	UFUNCTION(BlueprintNativeEvent, Category = "Advanced")
	UE_API FScriptSlateIcon GetIcon(const FToolMenuContext& Context) const;
	virtual FScriptSlateIcon GetIcon_Implementation(const FToolMenuContext& Context) const { return Data.Icon; }

	UFUNCTION(BlueprintImplementableEvent, Category = "Advanced")
	UE_API void ConstructMenuEntry(UToolMenu* Menu, const FName SectionName, const FToolMenuContext& Context);

	UFUNCTION(BlueprintCallable, Category = "Advanced")
	UE_API void RegisterMenuEntry();

	UFUNCTION(BlueprintCallable, Category = "Advanced")
	UE_API void UnregisterMenuEntry();

	UFUNCTION(BlueprintCallable, Category = "Advanced")
	UE_API void InitEntry(const FName OwnerName, const FName Menu, const FName Section, const FName Name, const FText& Label = FText(), const FText& ToolTip = FText());

	UFUNCTION(BlueprintNativeEvent, Category = "Advanced")
	UE_API bool ShowInToolbarTopLevel(const FToolMenuContext& Context) const;
	virtual bool ShowInToolbarTopLevel_Implementation(const FToolMenuContext& Context) const
	{
		return false;
	}

	FORCEINLINE bool CanSafelyRouteCall() { return !(GIntraFrameDebuggingGameThread || IsUnreachable() || FUObjectThreadContext::Get().IsRoutingPostLoad); }

	static UE_API UToolMenuEntryScript* GetIfCanSafelyRouteCall(const TWeakObjectPtr<UToolMenuEntryScript>& InWeak);

private:

	friend struct FToolMenuSection;
	friend class UToolMenus;
	friend class FPopulateMenuBuilderWithToolMenuEntry;

	UE_API TAttribute<FText> CreateLabelAttribute(FToolMenuContext& Context);

	UE_API TAttribute<FText> CreateToolTipAttribute(FToolMenuContext& Context);

	UE_API TAttribute<FSlateIcon> CreateIconAttribute(FToolMenuContext& Context);

	UE_API TAttribute<bool> CreateShowInToolbarTopLevelAttribute(FToolMenuContext& Context);

	UE_API void ToMenuEntry(FToolMenuEntry& Output);

	UE_API bool IsDynamicConstruct() const;

	UE_API FSlateIcon GetSlateIcon(const FToolMenuContext& Context) const;

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Data")
	FToolMenuEntryScriptData Data;

	UPROPERTY(Transient)
	bool bHasRegisteredMenuEntry = false;
};

#undef UE_API
