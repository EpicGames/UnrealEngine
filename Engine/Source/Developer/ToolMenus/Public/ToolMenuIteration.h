// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "ToolMenuContext.h"

class UToolMenu;
class UToolMenus;
struct FToolMenuEntry;
struct FToolMenuSection;

namespace UE::ToolMenus
{

struct FToolMenuIterationInfo
{
	FToolMenuIterationInfo(const UToolMenu& InMenu, const FToolMenuSection& InSection, const FToolMenuEntry& InEntry);

	const UToolMenu& Menu;
	const FToolMenuSection& Section;
	const FToolMenuEntry& Entry;
};

DECLARE_DELEGATE_RetVal_OneParam(bool, FToolMenuVisitor, const FToolMenuIterationInfo& IterationInfo);

void VisitMenuEntries(
	UToolMenus* InToolMenus, const FName InMenuName, const FToolMenuContext& InContext, const FToolMenuVisitor& InVisitor
);

} // namespace UE::ToolMenus
