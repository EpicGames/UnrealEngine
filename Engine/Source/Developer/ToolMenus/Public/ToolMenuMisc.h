// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/UIAction.h"

#include "ToolMenuMisc.generated.h"

#define UE_API TOOLMENUS_API

struct FToolMenuContext;

UENUM(BlueprintType)
enum class EToolMenuStringCommandType : uint8
{
	Command,
	Python,
	Custom
};

USTRUCT(BlueprintType, meta=(HasNativeBreak="/Script/ToolMenus.ToolMenuEntryExtensions.BreakStringCommand", HasNativeMake="/Script/ToolMenus.ToolMenuEntryExtensions.MakeStringCommand"))
struct FToolMenuStringCommand
{
	GENERATED_BODY()

	FToolMenuStringCommand() : Type(EToolMenuStringCommandType::Command) {}

	FToolMenuStringCommand(EToolMenuStringCommandType InType, FName InCustomType, const FString& InString) :
		Type(InType),
		CustomType(InCustomType),
		String(InString)
	{
	}

	// Which command handler to use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	EToolMenuStringCommandType Type;

	// Which command handler to use when type is custom
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName CustomType;

	// String to pass to command handler
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FString String;

private:

	friend class UToolMenus;

	bool IsBound() const { return String.Len() > 0; }

	UE_API FExecuteAction ToExecuteAction(FName MenuName, const FToolMenuContext& Context) const;

	UE_API FName GetTypeName() const;
};

UENUM(BlueprintType)
enum class EToolMenuInsertType : uint8
{
	Default,
	Before,
	After,
	First,
	Last
};

UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor="true"))
enum class EToolMenuInsertFallback : uint8
{
	None = 0 UMETA(Hidden),
	/** Insert the item even when the target is not found */
	Insert = 1 << 0,
	/** Log a warning if the target is not found */
	Log = 1 << 1,
};
ENUM_CLASS_FLAGS(EToolMenuInsertFallback);

USTRUCT(BlueprintType)
struct FToolMenuInsert
{
	GENERATED_BODY()

	FToolMenuInsert() : Position(EToolMenuInsertType::Default) {}
	FToolMenuInsert(FName InName, EToolMenuInsertType InPosition) : Name(InName), Position(InPosition) {}

	// Where to insert
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName Name;

	// How to insert
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	EToolMenuInsertType Position;
	
	// How to handle an item when the target insertion position is not found. Defaults to inserting the item while logging that the target was not found.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	EToolMenuInsertFallback Fallback = EToolMenuInsertFallback::Insert | EToolMenuInsertFallback::Log;

	FORCEINLINE bool operator==(const FToolMenuInsert& Other) const
	{
		return Other.Name == Name && Other.Position == Position;
	}

	FORCEINLINE bool operator!=(const FToolMenuInsert& Other) const
	{
		return Other.Name != Name || Other.Position != Position;
	}

	bool IsDefault() const
	{
		return Position == EToolMenuInsertType::Default;
	}

	bool IsBeforeOrAfter() const
	{
		return Position == EToolMenuInsertType::Before || Position == EToolMenuInsertType::After;
	}
};

#undef UE_API
