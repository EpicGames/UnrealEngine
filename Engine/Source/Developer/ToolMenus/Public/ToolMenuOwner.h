// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolMenuOwner.generated.h"

/**
 * Represents the owner of a menu. Can be used when registering menus
 * to later unregistering all menus created by a specified owner.
 *
 * See UToolMenus::UnregisterOwnerByName and FToolMenuOwnerScoped which relate to this.
 */
USTRUCT(BlueprintType, meta=(HasNativeBreak="/Script/ToolMenus.ToolMenuEntryExtensions.BreakToolMenuOwner", HasNativeMake="/Script/ToolMenus.ToolMenuEntryExtensions.MakeToolMenuOwner"))
struct FToolMenuOwner
{
	GENERATED_BODY()

private:
	struct FStoreName
	{
		int32 Index;
		int32 Number;
	};

	enum class EValueType : uint8
	{
		None,
		Pointer,
		Name
	};

public:
	FORCEINLINE FToolMenuOwner() : ValueInt64(0), ValueType(EValueType::None) {}
	FORCEINLINE FToolMenuOwner(void* InPointer) : ValueInt64(reinterpret_cast<int64>(InPointer)), ValueType(EValueType::Pointer) {}

	FToolMenuOwner(const WIDECHAR* InValue) : FToolMenuOwner(FName(InValue)) {}
	FToolMenuOwner(const ANSICHAR* InValue) : FToolMenuOwner(FName(InValue)) {}

	FToolMenuOwner(const FName InValue)
	{
		if (InValue == NAME_None)
		{
			ValueInt64 = 0;
			ValueType = EValueType::None;
		}
		else
		{
			ValueName.Index = InValue.GetComparisonIndex().ToUnstableInt();
			ValueName.Number = InValue.GetNumber();
			ValueType = EValueType::Name;
		}
	}

	FORCEINLINE bool operator==(const FToolMenuOwner& Other) const
	{
		return Other.ValueInt64 == ValueInt64 && Other.ValueType == ValueType;
	}

	FORCEINLINE bool operator!=(const FToolMenuOwner& Other) const
	{
		return Other.ValueInt64 != ValueInt64 || Other.ValueType != ValueType;
	}

	friend uint32 GetTypeHash(const FToolMenuOwner& Key)
	{
		return GetTypeHash(Key.ValueInt64);
	}

	FORCEINLINE bool IsSet() const { return ValueInt64 != 0; }

	FName TryGetName() const
	{
		if (ValueType == EValueType::Name)
		{
			const FNameEntryId EntryId = FNameEntryId::FromUnstableInt(ValueName.Index);
			return FName(EntryId, EntryId, ValueName.Number);
		}

		return NAME_None;
	}

private:

	union
	{
		int64 ValueInt64;
		FStoreName ValueName;
	};

	EValueType ValueType;
};
