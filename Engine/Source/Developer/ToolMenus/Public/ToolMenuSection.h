// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "Misc/Attribute.h"

#include "ToolMenuSection.generated.h"

#define UE_API TOOLMENUS_API

UCLASS(MinimalAPI, Blueprintable)
class UToolMenuSectionDynamic : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintImplementableEvent, Category = "Tool Menus")
	UE_API void ConstructSections(UToolMenu* Menu, const FToolMenuContext& Context);
};

UENUM(BlueprintType)
enum class EToolMenuSectionAlign : uint8
{
	Default,
	First,
	Middle,
	Last,
};

USTRUCT(BlueprintType)
struct FToolMenuSection
{
	GENERATED_BODY()

public:

	UE_API FToolMenuSection();

	UE_API void InitSection(const FName InName, const TAttribute< FText >& InLabel, const FToolMenuInsert InPosition);

	UE_API FToolMenuEntry& AddEntryObject(UToolMenuEntryScript* InObject);
	UE_API FToolMenuEntry& AddEntry(const FToolMenuEntry& Args);

	/**
	 * Add a menu entry to this section.
	 *
	 * @param InName The FName of the added entry used to identify it.
	 * @param InLabel The user-visible label of this entry.
	 * @param InToolTip The user-visible tooltip of this entry.
	 * @param InIcon The user-visible icon of this entry.
	 * @param InAction The delegates to call to make this entry interactive. Supports more than just on-click delegates. See FToolUIActionChoice for more information.
	 * @param UserInterfaceActionType The type of user interface element (such as a checkbox) to use for this entry. See EUserInterfaceActionType for more options.
	 * @param InTutorialHighlightName The tutorial highlight key
	 * @param InInputBindingOverride The key-binding reminder
	 * @return A reference to the added menu entry.
	 */
	UE_API FToolMenuEntry& AddMenuEntry(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const TAttribute<FSlateIcon>& InIcon, const FToolUIActionChoice& InAction, const EUserInterfaceActionType UserInterfaceActionType = EUserInterfaceActionType::Button, const FName InTutorialHighlightName = NAME_None, const TOptional<FText>& InInputBindingOverride = TOptional<FText>());
	UE_API FToolMenuEntry& AddMenuEntry(const TSharedPtr< const FUICommandInfo >& InCommand, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(), const FName InTutorialHighlightName = NAME_None, const TOptional<FName> InNameOverride = TOptional<FName>());
	UE_API FToolMenuEntry& AddMenuEntry(const FName InNameOverride, const TSharedPtr< const FUICommandInfo >& InCommand, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(), const FName InTutorialHighlightName = NAME_None);
	UE_API FToolMenuEntry& AddMenuEntryWithCommandList(const TSharedPtr<const FUICommandInfo>& InCommand, const TSharedPtr<const FUICommandList>& InCommandList, const TAttribute<FText>& InLabelOverride = TAttribute<FText>(), const TAttribute<FText>& InToolTipOverride = TAttribute<FText>(), const TAttribute<FSlateIcon>& InIconOverride = TAttribute<FSlateIcon>(), const FName InTutorialHighlightName = NAME_None, const TOptional<FName> InNameOverride = TOptional<FName>());

	UE_API FToolMenuEntry& AddDynamicEntry(const FName InName, const FNewToolMenuSectionDelegate& InConstruct);
	UE_API FToolMenuEntry& AddDynamicEntry(const FName InName, const FNewToolMenuDelegateLegacy& InConstruct);

	UE_API FToolMenuEntry& AddSeparator(const FName InName);

	/**
	 * Add a sub-menu to this section.
	 *
	 * @param InName The FName to use for identifying this sub-menu. Must be unique within the section.
	 * @param InLabel The user-visible label to use for this sub-menu.
	 * @param InToolTip The user-visible tooltip to use for this sub-menu.
	 * @param InMakeMenu The delegate to call to construct the sub-menu. This delegate is called when the user clicks the sub-menu entry.
	 * @param InAction The delegates to use to make this menu entry interactive.
	 * @param InUserInterfaceActionType The type of user interface element (such as a checkbox) to use for this entry.
	 * @param bInOpenSubMenuOnClick Whether or not the sub-menu entry opens when it is hovered or only when clicked.
	 * @param InIcon The user-visible icon to use for this sub-menu.
	 * @param bShouldCloseWindowAfterMenuSelection Whether or not the sub-menu should close after one of its entries is clicked.
	 * @return A reference to the menu entry for the added sub-menu.
	 */
	UE_API FToolMenuEntry& AddSubMenu(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewToolMenuChoice& InMakeMenu, const FToolUIActionChoice& InAction, const EUserInterfaceActionType InUserInterfaceActionType, bool bInOpenSubMenuOnClick = false, const TAttribute<FSlateIcon>& InIcon = TAttribute<FSlateIcon>(), const bool bShouldCloseWindowAfterMenuSelection = true);
	UE_API FToolMenuEntry& AddSubMenu(const FName InName, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip, const FNewToolMenuChoice& InMakeMenu, bool bInOpenSubMenuOnClick = false, const TAttribute<FSlateIcon>& InIcon = TAttribute<FSlateIcon>(), const bool bShouldCloseWindowAfterMenuSelection = true, const FName InTutorialHighlightName = NAME_None);
	UE_API FToolMenuEntry& AddSubMenu(const FName InName, const FToolUIActionChoice& InAction, const TSharedRef<SWidget>& InWidget, const FNewToolMenuChoice& InMakeMenu, bool bShouldCloseWindowAfterMenuSelection = true);

	template <typename TContextType>
	TContextType* FindContext() const
	{
		return Context.FindContext<TContextType>();
	}

	UE_API FToolMenuEntry* FindEntry(const FName InName);
	UE_API const FToolMenuEntry* FindEntry(const FName InName) const;

private:

	UE_API void InitGeneratedSectionCopy(const FToolMenuSection& Source, FToolMenuContext& InContext);

	UE_API int32 RemoveEntry(const FName InName);
	UE_API int32 RemoveEntryObject(const UToolMenuEntryScript* InObject);
	UE_API int32 RemoveEntriesByOwner(const FToolMenuOwner InOwner);

	UE_API int32 IndexOfBlock(const FName InName) const;
	UE_API int32 FindBlockInsertIndex(const FToolMenuEntry& InBlock) const;

	UE_API bool IsNonLegacyDynamic() const;

	UE_API bool IsRegistering() const;

	friend class UToolMenuSectionExtensions;
	friend class UToolMenus;
	friend class UToolMenu;

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FToolMenuOwner Owner;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	TArray<FToolMenuEntry> Blocks;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FToolMenuInsert InsertPosition;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	FToolMenuContext Context;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tool Menus")
	TObjectPtr<UToolMenuSectionDynamic> ToolMenuSectionDynamic;

	TAttribute<FText> Label;
	FToolMenuVisibilityChoice Visibility;

	FNewSectionConstructChoice Construct;

	EToolMenuSectionAlign Alignment;

	/** Section-level resize parameters. */
	FMenuEntryResizeParams ResizeParams;
	
	/** A sorting function for blocks within this section. Blocks are sorted after all entries have been added. */
	FToolMenuSectionSorter Sorter;

private:

	bool bIsRegistering;
	bool bAddedDuringRegister;
};

#undef UE_API
