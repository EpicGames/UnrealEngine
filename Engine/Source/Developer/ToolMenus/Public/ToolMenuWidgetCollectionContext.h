// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolMenuContext.h"

#include "ToolMenuWidgetCollectionContext.generated.h"

class SWidget;

/**
 * Context class containing a collection of widgets that can be added for later retrieval.
 * @see FToolMenuContext
 */
UCLASS(MinimalAPI)
class UToolMenuWidgetCollectionContext : public UToolMenuContextBase
{
	GENERATED_BODY()

public:
	/**
	 * Get the widget collection context object from the parameter context.
	 * @param Context The tool menu context to retrieve the context object from.
	 * @param bCreateIfNeeded If true (default) then the context object is created if one wasn't found.
	 * @return The context object, or null if one wasn't found and bCreateIfNeeded was false.
	 */
	[[nodiscard]] TOOLMENUS_API static UToolMenuWidgetCollectionContext* Get(FToolMenuContext& Context, bool bCreateIfNeeded = true);

	/**
	 * Add a widget to this context for later retrieval.
	 * @param Widget The widget to add to the context.
	 * @see FindContextWidget
	 */
	TOOLMENUS_API void AddWidget(const TSharedRef<SWidget>& Widget);

	/**
	 * Find a previously added widget. Uses the widget's type via WidgetType::StaticWidgetClass.
	 * @tparam WidgetType The type of widget to find.
	 * @return The widget, if found. A null pointer otherwise.
	 * @note This requires that the widget implements SLATE_DECLARE_WIDGET in its class definition.
	 * @note This function performs an exact match on the widget type - there is no inheritance checking.
	 * @see AddWidget
	 */
	template <typename WidgetType>
	[[nodiscard]] TSharedPtr<WidgetType> FindWidget() const
	{
		const FName TypeToFind = WidgetType::StaticWidgetClass().GetWidgetType();
		return StaticCastSharedPtr<WidgetType>(FindWidgetByClassType(TypeToFind));
	}

	/**
	 * Enumerate all the previously added widgets.
	 * @param InCallback Callback invoked for each wdget; return true to continue enumeration, or false to stop.
	 */
	TOOLMENUS_API void EnumerateWidgets(const TFunctionRef<bool(const TSharedPtr<SWidget>& Widget)>& Callback) const;

private:
	[[nodiscard]] TOOLMENUS_API TSharedPtr<SWidget> FindWidgetByClassType(const FName& WidgetClassType) const;

	TArray<TWeakPtr<SWidget>> Widgets;
};
