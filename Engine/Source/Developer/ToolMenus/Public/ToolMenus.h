// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "Misc/Attribute.h"

#include "IToolMenusModule.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenuEntryScript.h"
#include "ToolMenuOwner.h"
#include "ToolMenuSection.h"
#include "ToolMenuMisc.h"

#include "Misc/CoreDelegates.h"

#include "ToolMenus.generated.h"

#define UE_API TOOLMENUS_API

class FMenuBuilder;
class FMultiBox;
class SWidget;

namespace UE::ToolMenus
{
class FToolMenuTestInstanceScoped;
}
class UToolMenu;
class UToolMenuEntryScript;
struct FToolMenuEntry;
struct FToolMenuSection;

struct FGeneratedToolMenuWidget
{
	// A copy of the menu so we can refresh menus not in the database
	TObjectPtr<UToolMenu> GeneratedMenu;

	// The actual widget for the menu
	TWeakPtr<SWidget> Widget;

	// Weak ptr to the original menu that owns the widget
	TWeakObjectPtr<UToolMenu> OriginalMenu;
};

struct FGeneratedToolMenuWidgets
{
	TArray<TSharedPtr<FGeneratedToolMenuWidget>> Instances;
};

namespace UE::ToolMenus
{
	/**
	 * A handle to a particular entry. Used when elevating child elements to a parent toolbar.
	 */
	struct FSubBlockReference
	{
		FSubBlockReference();
		FSubBlockReference(const FSubBlockReference& Other);
		FSubBlockReference(UToolMenu* InParent, FToolMenuSection* InSection, FToolMenuEntry* InEntry);

		UToolMenu* ParentMenu;
		FToolMenuSection* Section;
		FToolMenuEntry* Entry;
	};	
}

/*
 * A global context that any menu can add/modify to specify which profiles are currently active
 */
UCLASS(MinimalAPI)
class UToolMenuProfileContext : public UToolMenuContextBase
{
	GENERATED_BODY()
public:

	TArray<FName> ActiveProfiles;
};

/*
 * Struct to store all the profiles for a menu for serialization
 */
USTRUCT()
struct FToolMenuProfileMap
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FName /*profile name*/, FToolMenuProfile> MenuProfiles;
};

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UToolMenus : public UObject
{
	GENERATED_BODY()

public:

	UE_API UToolMenus();

	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	static UE_API UToolMenus* Get();

	/** Try to get UToolMenus without forcing ToolMenus module to load. */
	static inline UToolMenus* TryGet()
	{
		if (IToolMenusModule::IsAvailable())
		{
			return Get();
		}

		return nullptr;
	}

	/** Unregister everything associated with the given owner without forcing ToolMenus module to load. */
	static inline void UnregisterOwner(FToolMenuOwner Owner)
	{
		if (UToolMenus* ToolMenus = UToolMenus::TryGet())
		{
			ToolMenus->UnregisterOwnerInternal(Owner);
		}
	}

	/**
	 * Returns true if slate initialized and editor GUI is being used.
	 * The application should have been initialized before this method is called.
	 *
	 * @return	True if slate initialized and editor GUI is being used.
	 */
	static UE_API bool IsToolMenuUIEnabled();

	/** 
	 * Delays menu registration until safe and ready
	 * Will not trigger if Slate does not end up being enabled after loading
	 * Will not trigger when running commandlet, game, dedicated server or client only
	 *
	 */
	static UE_API FDelegateHandle RegisterStartupCallback(const FSimpleMulticastDelegate::FDelegate& InDelegate);

	/** Unregister a startup callback delegate by pointer */
	static UE_API void UnRegisterStartupCallback(FDelegateUserObjectConst UserPointer);

	/** Unregister a startup callback delegate by handle */
	static UE_API void UnRegisterStartupCallback(FDelegateHandle InHandle);

	/**
	 * Registers a menu by name
	 * @param	Parent	Optional name of a menu to layer on top of.
	 * @param	Type	Type of menu that will be generated such as: ToolBar, VerticalToolBar, etc..
	 * @param	bWarnIfAlreadyRegistered	Display warning if already registered
	 * @return	ToolMenu	Menu object
	 */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API UToolMenu* RegisterMenu(FName Name, const FName Parent = NAME_None, EMultiBoxType Type = EMultiBoxType::Menu, bool bWarnIfAlreadyRegistered = true);

	/**
	 * Extends a menu without registering the menu or claiming ownership of it. Ok to call even if menu does not exist yet.
	 * @param	Name	Name of the menu to extend
	 * @return	ToolMenu	Menu object
	 */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API UToolMenu* ExtendMenu(const FName Name);

	/**
	 * Generate widget from a registered menu. Most common function used to generate new menu widgets.
	 * @param	Name	Registered menu's name that widget will be generated from
	 * @param	Context	Additional information specific to the menu being generated
	 * @return	Widget to display
	 */
	UE_API TSharedRef<SWidget> GenerateWidget(const FName Name, const FToolMenuContext& InMenuContext);

	/**
	 * Finds an existing menu that has been registered or extended.
	 * @param	Name	Name of the menu to find.
	 * @return	ToolMenu	Menu object. Returns null if not found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API UToolMenu* FindMenu(const FName Name);

	/**
	 * Determines if a menu has already been registered.
	 * @param	Name	Name of the menu to find.
	 * @return	bool	True if menu has already been registered.
	 */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API bool IsMenuRegistered(const FName Name) const;

	/** Rebuilds all widgets generated from a specific menu. */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API bool RefreshMenuWidget(const FName Name);

	/** Rebuilds all currently generated widgets next tick. */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void RefreshAllWidgets();

	/** Registers menu entry object from blueprint/script */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	static UE_API bool AddMenuEntryObject(UToolMenuEntryScript* MenuEntryObject);

	/** Unregisters menu entry object from blueprint/script */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	static UE_API bool RemoveMenuEntryObject(UToolMenuEntryScript* MenuEntryObject);

	/** Removes all entries that were registered under a specific owner name */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void UnregisterOwnerByName(FName InOwnerName);

	/** Sets a section's displayed label text. */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void SetSectionLabel(const FName MenuName, const FName SectionName, const FText Label);

	/** Sets where to insert a section into a menu when generating relative to other section names. */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void SetSectionPosition(const FName MenuName, const FName SectionName, const FName OtherSectionName, const EToolMenuInsertType PositionType);

	/** Registers a section for a menu */
	UE_API void AddSection(const FName MenuName, const FName SectionName, const TAttribute< FText >& InLabel, const FToolMenuInsert InPosition = FToolMenuInsert());

	/** Registers an entry for a menu's section */
	UE_API void AddEntry(const FName MenuName, const FName SectionName, const FToolMenuEntry& Entry);

	/** Removes a menu entry from a given menu and section */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void RemoveEntry(const FName MenuName, const FName Section, const FName Name);

	/** Removes a section from a given menu */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void RemoveSection(const FName MenuName, const FName Section);

	/** Unregisters a menu by name */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	UE_API void RemoveMenu(const FName MenuName);

	/** Finds a context object of a given class if it exists */
	UFUNCTION(BlueprintCallable, Category = "Tool Menus")
	static UE_API UObject* FindContext(const FToolMenuContext& InContext, UClass* InClass);

	/**
	 * Generate widget from a hierarchy of menus. For advanced specialized use cases.
	 * @param	Hierarchy	Array of menus to combine into a final widget
	 * @param	Context	Additional information specific to the menu being generated
	 * @return	Widget to display
	 */
	UE_API TSharedRef<SWidget> GenerateWidget(const TArray<UToolMenu*>& Hierarchy, const FToolMenuContext& InMenuContext);

	/**
	 * Generate widget from a final collapsed menu. For advanced specialized use cases.
	 * @param	GeneratedMenu	Combined final menu to generate final widget from
	 * @return	Widget to display
	 */
	UE_API TSharedRef<SWidget> GenerateWidget(UToolMenu* GeneratedMenu);
	
	/** Create a finalized menu that combines all parents used to generate a widget. Advanced special use cases only. */
	UE_API UToolMenu* GenerateMenu(const FName Name, const FToolMenuContext& InMenuContext);

	/** Create a finalized menu based on a custom crafted menu. Advanced special use cases only. */
	UE_API UToolMenu* GenerateMenuAsBuilder(const UToolMenu* InMenu, const FToolMenuContext& InMenuContext);

	/* Generate either a menu or submenu ready for editing */
	UE_API UToolMenu* GenerateMenuOrSubMenuForEdit(const UToolMenu* InMenu);

	/** Bake final menu including calls to construction delegates, sorting, and customization */
	UE_API void AssembleMenuHierarchy(UToolMenu* GeneratedMenu, const TArray<UToolMenu*>& Hierarchy);

	/* Returns list of menus starting with root parent */
	UE_API TArray<UToolMenu*> CollectHierarchy(const FName Name);

	/** For advanced use cases */
	UE_API FToolMenuOwner CurrentOwner() const;

	/** Registers a new type of string based command handler. */
	UE_API void RegisterStringCommandHandler(const FName InName, const FToolMenuExecuteString& InDelegate);

	/** Removes a string based command handler. */
	UE_API void UnregisterStringCommandHandler(const FName InName);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStringCommandExecuted, FName MenuName, FName TypeName);
	/** Called when we execute a string command. Useful for analytics */
	FOnStringCommandExecuted OnStringCommandExecuted;
	
	/** Sets delegate to setup timer for deferred one off ticks */
	UE_API void AssignSetTimerForNextTickDelegate(const FSimpleDelegate& InDelegate);

	/** Timer function used to consolidate multiple duplicate requests into a single frame. */
	UE_API void HandleNextTick();

	/** Remove customization for a menu */
	UE_API void RemoveCustomization(const FName InName);

	/** Remove all menu customizations for all menus */
	UE_API void RemoveAllCustomizations();

	/** Save menu customizations to ini files */
	UE_API void SaveCustomizations();

	/** Find customization settings for a menu */
	UE_API FCustomizedToolMenu* FindMenuCustomization(const FName InName);

	/** Find or add customization settings for a menu */
	UE_API FCustomizedToolMenu* AddMenuCustomization(const FName InName);

	/** Find index of customization settings for a menu */
	UE_API int32 FindMenuCustomizationIndex(const FName InName);

	/** Find runtime customization settings for a menu */
	UE_API FCustomizedToolMenu* FindRuntimeMenuCustomization(const FName InName);

	/** Find or add runtime customization settings for a menu */
	UE_API FCustomizedToolMenu* AddRuntimeMenuCustomization(const FName InName);

	/** Find a specific profile for a menu */
	UE_API FToolMenuProfile* FindMenuProfile(const FName InMenuName, const FName InProfileName);

	/** Find or add a specific profile for a menu */
	UE_API FToolMenuProfile* AddMenuProfile(const FName InMenuName, const FName InProfileName);

	/** Find a specific runtime only profile for a menu */
	UE_API FToolMenuProfile* FindRuntimeMenuProfile(const FName InMenuName, const FName InProfileName);

	/** Find or add a specific runtime only profile for a menu */
	UE_API FToolMenuProfile* AddRuntimeMenuProfile(const FName InMenuName, const FName InProfileName);

	/** Unregister runtime customization settings for a specific owner name */
	UE_API void UnregisterRuntimeMenuCustomizationOwner(const FName InOwnerName);

	/** Unregister runtime profile settings for a specific owner name */
	UE_API void UnregisterRuntimeMenuProfileOwner(const FName InOwnerName);

	/** Generates sub menu by entry name in the given generated menu parent */
	UE_API UToolMenu* GenerateSubMenu(const UToolMenu* InGeneratedParent, const FName InBlockName);

	/** When true, adds command to open edit menu dialog to each menu */
	UE_API bool GetEditMenusMode() const;

	/** Enables adding command to open edit menu dialog to each menu */
	UE_API void SetEditMenusMode(bool bEnable);

	/* Substitute one menu for another during generate but not during find or extend */
	UE_API void AddMenuSubstitutionDuringGenerate(const FName OriginalMenu, const FName NewMenu);

	/* Remove substitute one menu for another during generate */
	UE_API void RemoveSubstitutionDuringGenerate(const FName InMenu);

	/** Release references to UObjects of widgets that have been deleted. Combines multiple requests in one frame together for improved performance. */
	UE_API void CleanupStaleWidgetsNextTick(bool bGarbageCollect = false);

	/** Displaying extension points is for debugging menus */
	DECLARE_DELEGATE_RetVal(bool, FShouldDisplayExtensionPoints);
	FShouldDisplayExtensionPoints ShouldDisplayExtensionPoints;

	/* Delegate that opens a menu editor */
	DECLARE_DELEGATE_OneParam(FEditMenuDelegate, class UToolMenu*);
	FEditMenuDelegate EditMenuDelegate;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FGenerateWidgetEvent, const FName InName, const FToolMenuContext& InMenuContext);
	/** Called before we generate a menu widget. */
	FGenerateWidgetEvent OnPreGenerateWidget;
	/** Called after we generate a menu widget. */
	FGenerateWidgetEvent OnPostGenerateWidget;

	/** Icon to display in menus for command to open menu editor */
	FSlateIcon EditMenuIcon;

	/** Icon to display in toolbars for command to open menu editor */
	FSlateIcon EditToolbarIcon;

	/** Join two paths together */
	static UE_API FName JoinMenuPaths(const FName Base, const FName Child);

	/** Break apart a menu path into components */
	static UE_API bool SplitMenuPath(const FName MenuPath, FName& OutLeft, FName& OutRight);

	/** Returns true if safe to call into script */
	static UE_API bool CanSafelyRouteCall();

	friend struct FToolMenuOwnerScoped;
	friend struct FToolMenuStringCommand;

	//~ Begin UObject Interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	UE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

private:
	friend class FPopulateMenuBuilderWithToolMenuEntry;

	/** Create a finalized menu that combines given hierarchy array that will generate a widget. Advanced special use cases only. */
	UE_API UToolMenu* GenerateMenuFromHierarchy(const TArray<UToolMenu*>& Hierarchy, const FToolMenuContext& InMenuContext);

	/** Sets a timer to be called next engine tick so that multiple repeated actions can be combined together. */
	UE_API void SetNextTickTimer();

	/** Release references to UObjects of widgets that have been deleted */
	UE_API void CleanupStaleWidgets();

	/** Re-creates widget that is active */
	UE_API bool RefreshMenuWidget(const FName Name, FGeneratedToolMenuWidget& GeneratedMenuWidget);

	/** Sets the current temporary menu owner to avoid needing to supply owner for each menu entry being registered. Used by FToolMenuEntryScoped */
	UE_API void PushOwner(const FToolMenuOwner InOwner);

	/** Sets the current temporary menu owner. Used by FToolMenuEntryScoped */
	UE_API void PopOwner(const FToolMenuOwner InOwner);

	static UE_API void PrivateStartupCallback();
	static UE_API void UnregisterPrivateStartupCallback();

	UE_API UToolMenu* FindSubMenuToGenerateWith(const FName InParentName, const FName InChildName);

	UE_API void PopulateMenuBuilder(FMenuBuilder& MenuBuilder, UToolMenu* MenuData);
	UE_API void PopulateMenuBarBuilder(FMenuBarBuilder& MenuBarBuilder, UToolMenu* MenuData);

	UE_API void ExtractChildBlocksFromSubMenu(UToolMenu* ParentMenu, FToolMenuEntry& InBlock, TArray<UE::ToolMenus::FSubBlockReference>& SubMenuBlocks);
	UE_API void PopulateToolBarBuilderWithTopLevelChildren(FToolBarBuilder& ToolBarBuilder, UToolMenu* ParentMenu, FToolMenuEntry& InBlock, bool bAddSpaceAfterRaisedChildren);
	UE_API void PopulateToolBarBuilderWithEntry(
		FToolBarBuilder& ToolBarBuilder,
		UToolMenu* MenuData,
		FToolMenuSection& Section,
		FToolMenuEntry& Entry,
		bool bIsRaisingToTopLevel = false,
		bool bIsLastBlockOfLastSection = false
	);
	UE_API void PopulateToolBarBuilder(FToolBarBuilder& ToolBarBuilder, UToolMenu* MenuData);

	UE_API TSharedRef<SWidget> GenerateToolbarComboButtonMenu(TWeakObjectPtr<UToolMenu> InParent, const FName InBlockName);

	UE_API FOnGetContent ConvertWidgetChoice(const FNewToolMenuChoice& Choice, const FToolMenuContext& Context, const FToolMenuEntry& Block) const;

	/** Converts a string command to a FUIAction */
	static UE_API FUIAction ConvertUIAction(const FToolMenuEntry& Block, const FToolMenuContext& Context);
	static UE_API FUIAction ConvertUIAction(const FToolUIActionChoice& Choice, const FToolMenuContext& Context);
	static UE_API FUIAction ConvertUIAction(const FToolUIAction& Actions, const FToolMenuContext& Context);
	static UE_API FUIAction ConvertUIAction(const FToolDynamicUIAction& Actions, const FToolMenuContext& Context);
	static UE_API FUIAction ConvertScriptObjectToUIAction(UToolMenuEntryScript* ScriptObject, const FToolMenuContext& Context);
	
	static UE_API TAttribute<EVisibility> CalculateToolbarVisibility(
		UToolMenu* Menu,
		FToolMenuSection& Section,
		FToolMenuEntry& Entry,
		bool bIsRaisingToTopLevel,
		bool bEmbedActionOrCommand
	);

	static UE_API void ExecuteStringCommand(const FToolMenuStringCommand StringCommand, FName MenuName, const FToolMenuContext Context);

	UE_API void PopulateSubMenu(FMenuBuilder& Builder, TWeakObjectPtr<UToolMenu> InParent, const FToolMenuEntry InEntry, const FName InBlockName);
	UE_API void PopulateSubMenuWithoutName(FMenuBuilder& MenuBuilder, TWeakObjectPtr<UToolMenu> InParent, const FToolMenuEntry InEntry);
	UE_API TArray<UToolMenu*> CollectHierarchy(const FName Name, const TMap<FName, FName>& UnregisteredParentNames);

	/** Applies the style to the given Builder, as specified by the provided Style. */
	UE_API void ApplyStyleToBuilder(FMenuBuilder& InBuilder, const FToolMenuEntryStyle& InStyle);

	UE_API void ListAllParents(const FName Name, TArray<FName>& AllParents);

	UE_API void AssembleMenu(UToolMenu* GeneratedMenu, const UToolMenu* Other);
	UE_API void AssembleMenuSection(UToolMenu* GeneratedMenu, const UToolMenu* Other, FToolMenuSection* DestSection, const FToolMenuSection& OtherSection);

	UE_API void CopyMenuSettings(UToolMenu* GeneratedMenu, const UToolMenu* Other);

	UE_API void AddReferencedContextObjects(const TSharedRef<FMultiBox>& InMultiBox, const UToolMenu* InMenu);

	UE_API void ApplyCustomizationAndProfiles(UToolMenu* GeneratedMenu);
	UE_API void ApplyProfile(UToolMenu* GeneratedMenu, const FToolMenuProfile& Profile);
	UE_API void ApplyCustomization(UToolMenu* GeneratedMenu, const FCustomizedToolMenu& Profile);

	UE_API void UnregisterOwnerInternal(FToolMenuOwner Owner);

	UE_API bool GetDisplayUIExtensionPoints() const;

	static UE_API void ModifyEntryForEditDialog(FToolMenuEntry& Entry);

	UE_API UToolMenu* NewToolMenuObject(const FName NewBaseName, const FName InMenuName);

private:

	UPROPERTY(EditAnywhere, Category = Misc)
	TArray<FCustomizedToolMenu> CustomizedMenus;
	
	/* Allow substituting one menu for another during generate but not during find or extend */
	UPROPERTY(EditAnywhere, Category = Misc)
	TMap<FName, FName> MenuSubstitutionsDuringGenerate;

	UPROPERTY()
	TMap<FName, TObjectPtr<UToolMenu>> Menus;

	TMap<FName, TSharedPtr<FGeneratedToolMenuWidgets>> GeneratedMenuWidgets;

	TMap<TWeakPtr<FMultiBox>, TArray<TObjectPtr<const UObject>>> WidgetObjectReferences;

	TArray<FToolMenuOwner> OwnerStack;

	TMap<FName, FToolMenuExecuteString> StringCommandHandlers;

	/** Transient customizations made during runtime that will not be saved */
	TArray<FCustomizedToolMenu> RuntimeCustomizedMenus;

	UPROPERTY(EditAnywhere, Category = Misc)
	TMap<FName /*MenuName*/, FToolMenuProfileMap> MenuProfiles;

	/** Transient profiles made during runtime that will not be saved */
	TMap<FName /*MenuName*/, FToolMenuProfileMap> RuntimeMenuProfiles;

	FSimpleDelegate SetTimerForNextTickDelegate;

	bool bNextTickTimerIsSet;
	bool bRefreshWidgetsNextTick;
	bool bCleanupStaleWidgetsNextTick;
	bool bCleanupStaleWidgetsNextTickGC;
	bool bEditMenusMode;
	bool bSuppressRefreshWidgetsRequests = false;

	static UE_API UToolMenus* Singleton;
	static UE_API bool bHasShutDown;
	static UE_API FSimpleMulticastDelegate StartupCallbacks;
	static UE_API TOptional<FDelegateHandle> InternalStartupCallbackHandle;

	friend class UE::ToolMenus::FToolMenuTestInstanceScoped;
};

/**
 * Sets the owner for all menus created until the end of the current scope (with support for nested scopes).
 * Combines well with UToolMenus::UnregisterOwnerByName.
 */
struct FToolMenuOwnerScoped
{
	FToolMenuOwnerScoped(const FToolMenuOwner InOwner) : Owner(InOwner)
	{
		UToolMenus::Get()->PushOwner(InOwner);
	}

	~FToolMenuOwnerScoped()
	{
		UToolMenus::Get()->PopOwner(Owner);
	}

	FToolMenuOwner GetOwner() const { return Owner; }

private:

	FToolMenuOwner Owner;
};

namespace UE::ToolMenus
{

/**
 * Creates a new instance of UToolMenus and sets the UToolMenus singleton to point at this new instance.
 * When the scope instance is destructed, the previous UToolMenus singleton is reinstated.
 */
class FToolMenuTestInstanceScoped
{
public:
	FToolMenuTestInstanceScoped();
	~FToolMenuTestInstanceScoped();

private:
	UToolMenus* PreviousInstance;
	UToolMenus* ScopedInstance;
};

}; // namespace UE::ToolMenus

#undef UE_API
